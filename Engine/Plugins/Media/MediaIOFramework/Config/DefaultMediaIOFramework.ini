[CoreRedirects]

; 4.20
+EnumRedirects=(OldName="EMediaIOSDITransportType", NewName="/Script/MediaIOCore.EMediaIOTransportType")