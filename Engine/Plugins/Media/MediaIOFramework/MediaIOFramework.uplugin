{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Media IO Framework",
	"Description": "Media Framework classes to support Professional Media IO used by the Virtual Production industry.",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "MediaIOCore",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		},
		{
			"Name": "MediaIOEditor",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		},
		{
			"Name": "GPUTextureTransfer",
			"Type": "Runtime",
			"LoadingPhase": "EarliestPossible",
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		}
	],
	"Plugins": [
		{
			"Name": "OpenColorIO",
			"Enabled": true
		}
	],
	"DocsURL": "",
	"IsExperimentalVersion": false
}