// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/ScreenPass.ush"
#include "/Engine/Private/Common.ush"

SCREEN_PASS_TEXTURE_VIEWPORT(Input)
SCREEN_PASS_TEXTURE_VIEWPORT(Output)

Texture2D InputTexture;

SamplerState InputSampler;

// Shader to downsample a texture, Adapted from PostProcessDownsample.usf
void MediaDownsample(float4 SvPosition : SV_POSITION, out float4 OutColor : SV_Target0)
{
	const float2 UV = SvPosition.xy * Output_ExtentInverse;

	// Output: float4(RGBA), 4 filtered samples
	float2 UVs[4];

	// Blur during downsample (4x4 kernel) to get better quality especially for HDR content.
	UVs[0] = UV + Input_ExtentInverse * float2(-1, -1);
	UVs[1] = UV + Input_ExtentInverse * float2( 1, -1);
	UVs[2] = UV + Input_ExtentInverse * float2(-1,  1);
	UVs[3] = UV + Input_ExtentInverse * float2( 1,  1);

	float4 Sample[4];

	UNROLL
	for(uint i = 0; i < 4; ++i)
	{
		float2 UVAtIndex = clamp(UVs[i], Input_UVViewportBilinearMin, Input_UVViewportBilinearMax);
		Sample[i] = Texture2DSampleLevel(InputTexture, InputSampler, UVAtIndex, 0);
	}

	OutColor = (Sample[0] + Sample[1] + Sample[2] + Sample[3]) * 0.25f;

	// Fixed rarely occurring yellow color tint of the whole viewport (certain viewport size, need to investigate more)
	OutColor.rgb = max(float3(0,0,0), OutColor.rgb);
}

float4 RotationScale;

void MediaTransform(float4 SvPosition : SV_POSITION, out float4 OutColor : SV_Target0)
{
	const float2 Position = SvPosition.xy - 0.5 * Output_Extent;
	const float2 TransformedPosition = float2(dot(Position, RotationScale.xy), dot(Position, RotationScale.zw));
	const float2 UV = TransformedPosition * Input_ExtentInverse + float2(0.5, 0.5);
	
	OutColor = Texture2DSampleLevel(InputTexture, InputSampler, UV, 0);
}