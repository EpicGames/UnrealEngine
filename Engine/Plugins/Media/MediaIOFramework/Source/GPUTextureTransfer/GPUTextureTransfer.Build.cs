// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class GPUTextureTransfer : ModuleRules
	{
		public GPUTextureTransfer(ReadOnlyTargetRules Target) : base(Target)
		{
			// no dvp.lib for Arm64
			if (Target.Platform == UnrealTargetPlatform.Win64 && Target.Architecture != UnrealArch.Arm64)
			{
				AddEngineThirdPartyPrivateStaticDependencies(Target, "GPUDirect");
				PublicDefinitions.Add("DVP_SUPPORTED_PLATFORM=1");

				if (Target.bCompileAgainstEngine)
				{
					AddEngineThirdPartyPrivateStaticDependencies(Target, "Vulkan", "DX11", "DX12");
					PrivateDependencyModuleNames.AddRange(["VulkanRHI", "D3D11RHI", "D3D12RHI"]);
				}
				else
				{
					PublicDefinitions.Add("DVP_SUPPORTED_PLATFORM=0");
				}
			}
			else if (Target.Platform == UnrealTargetPlatform.Linux || Target.Platform == UnrealTargetPlatform.LinuxArm64)
			{
				AddEngineThirdPartyPrivateStaticDependencies(Target, "Vulkan");
				PrivateDependencyModuleNames.Add("VulkanRHI");
				PublicDefinitions.Add("DVP_SUPPORTED_PLATFORM=0");
			}
			else
			{
				PublicDefinitions.Add("DVP_SUPPORTED_PLATFORM=0");
			}

			PublicDependencyModuleNames.AddRange(
            	new string[]
                {
                    "Core",
                    "RHI"
                });

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
                	"GPUDirect",
					"RenderCore",
				});

			PublicDefinitions.Add("PERF_LOGGING=0");

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.Add("EditorFramework");
				PrivateDependencyModuleNames.Add("UnrealEd");
			}
		}

	}
}
