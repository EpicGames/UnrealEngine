// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D11TextureTransfer.h"

#if DVP_SUPPORTED_PLATFORM
#include "Windows/AllowWindowsPlatformTypes.h"
	#include "dvpapi_d3d11.h"
#include "Windows/HideWindowsPlatformTypes.h"

#include "ID3D11DynamicRHI.h"

namespace UE::GPUTextureTransfer::Private
{
DVPStatus FD3D11TextureTransfer::Init_Impl()
{
	D3DDevice = GetID3D11DynamicRHI()->RHIGetDevice();
	return dvpInitD3D11Device(D3DDevice, 0);
}

DVPStatus FD3D11TextureTransfer::GetConstants_Impl(uint32* OutBufferAddrAlignment, uint32* OutBufferGPUStrideAlignment, uint32* OutSemaphoreAddrAlignment, uint32* OutSemaphoreAllocSize, uint32* OutSemaphorePayloadOffset, uint32* OutSemaphorePayloadSize) const
{
	if (!D3DDevice)
	{
		return DVP_STATUS_ERROR;
	}

	return dvpGetRequiredConstantsD3D11Device(OutBufferAddrAlignment, OutBufferGPUStrideAlignment, OutSemaphoreAddrAlignment, OutSemaphoreAllocSize,
		OutSemaphorePayloadOffset, OutSemaphorePayloadSize, D3DDevice);
}

DVPStatus FD3D11TextureTransfer::CloseDevice_Impl() const
{
	if (!D3DDevice)
	{
		return DVP_STATUS_ERROR;
	}

	return dvpCloseD3D11Device(D3DDevice);
}

DVPStatus FD3D11TextureTransfer::BindBuffer_Impl(DVPBufferHandle InBufferHandle) const
{
	if (!D3DDevice)
	{
		return DVP_STATUS_ERROR;
	}

	return dvpBindToD3D11Device(InBufferHandle, D3DDevice);
}

DVPStatus FD3D11TextureTransfer::CreateGPUResource_Impl(const FRegisterDMATextureArgs& InArgs, FTextureInfo* OutTextureInfo) const
{
	if (!OutTextureInfo || !D3DDevice)
	{
		return DVP_STATUS_ERROR;
	}

	OutTextureInfo->External.Handle = nullptr;
	return dvpCreateGPUD3D11Resource((ID3D11Resource*)InArgs.RHITexture->GetNativeResource(), &OutTextureInfo->DVPHandle);
}

DVPStatus FD3D11TextureTransfer::UnbindBuffer_Impl(DVPBufferHandle InBufferHandle) const
{
	if (!D3DDevice)
	{
		return DVP_STATUS_ERROR;
	}

	return dvpUnbindFromD3D11Device(InBufferHandle, D3DDevice);
}

}

#endif // DVP_SUPPORTED_PLATFORM
