// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TextureTransferBase.h"

#if DVP_SUPPORTED_PLATFORM
struct ID3D11Device;

namespace UE::GPUTextureTransfer::Private
{
class FD3D11TextureTransfer : public FTextureTransferBase
{
protected:
	virtual DVPStatus Init_Impl() override;
	virtual DVPStatus GetConstants_Impl(uint32* OutBufferAddrAlignment, uint32* OutBufferGPUStrideAlignment, uint32* OutSemaphoreAddrAlignment, uint32* OutSemaphoreAllocSize, uint32* OutSemaphorePayloadOffset, uint32* OutSemaphorePayloadSize) const override;
	virtual DVPStatus BindBuffer_Impl(DVPBufferHandle InBufferHandle) const override;
	virtual DVPStatus CreateGPUResource_Impl(const FRegisterDMATextureArgs& InArgs, FTextureInfo* OutTextureInfo) const override;
	virtual DVPStatus UnbindBuffer_Impl(DVPBufferHandle InBufferHandle) const override;
	virtual DVPStatus CloseDevice_Impl() const override;

private:
	ID3D11Device* D3DDevice = nullptr;
};
}

#endif // DVP_SUPPORTED_PLATFORM