// Copyright Epic Games, Inc. All Rights Reserved.

#include "D3D12TextureTransfer.h"

#if DVP_SUPPORTED_PLATFORM
#include "Windows/AllowWindowsPlatformTypes.h"
	#include "dvpapi_d3d12.h"
#include "Windows/HideWindowsPlatformTypes.h"

#include "ID3D12DynamicRHI.h"

namespace UE::GPUTextureTransfer::Private
{
	DVPStatus FD3D12TextureTransfer::Init_Impl()
	{
		ID3D12DynamicRHI* D3D12RHI = GetID3D12DynamicRHI();

		D3DDevice = D3D12RHI->RHIGetDevice_NoMGPU();
		D3DCommandQueue = D3D12RHI->RHIGetCommandQueue();

		return dvpInitD3D12Device(D3DDevice, 0);
	}

	DVPStatus FD3D12TextureTransfer::GetConstants_Impl(uint32* OutBufferAddrAlignment, uint32* OutBufferGPUStrideAlignment, uint32* OutSemaphoreAddrAlignment, uint32* OutSemaphoreAllocSize, uint32* OutSemaphorePayloadOffset, uint32* OutSemaphorePayloadSize) const
	{
		if (!D3DDevice)
		{
			return DVP_STATUS_ERROR;
		}

		return dvpGetRequiredConstantsD3D12Device(OutBufferAddrAlignment, OutBufferGPUStrideAlignment, OutSemaphoreAddrAlignment, OutSemaphoreAllocSize,
			OutSemaphorePayloadOffset, OutSemaphorePayloadSize, D3DDevice);
	}

	DVPStatus FD3D12TextureTransfer::CloseDevice_Impl() const
	{
		if (!D3DDevice)
		{
			return DVP_STATUS_ERROR;
		}

		return dvpCloseD3D12Device(D3DDevice);
	}

	DVPStatus FD3D12TextureTransfer::BindBuffer_Impl(DVPBufferHandle InBufferHandle) const
	{
		if (!D3DDevice)
		{
			return DVP_STATUS_ERROR;
		}

		return dvpBindToD3D12Device(InBufferHandle, D3DDevice);
	}

	DVPStatus FD3D12TextureTransfer::CreateGPUResource_Impl(const FRegisterDMATextureArgs& InArgs, FTextureTransferBase::FTextureInfo* OutTextureInfo) const
	{
		if (!OutTextureInfo || !D3DDevice || !InArgs.Stride)
		{
			return DVP_STATUS_ERROR;
		}

		ID3D12Resource* D3D12Texture = (ID3D12Resource*) InArgs.RHITexture->GetNativeResource();
		D3D12_RESOURCE_DESC ResourceDesc = D3D12Texture->GetDesc();

		HRESULT Res = D3DDevice->CreateSharedHandle(D3D12Texture, NULL, GENERIC_ALL, NULL, &OutTextureInfo->External.Handle);
		if (FAILED(Res))
		{
			return DVP_STATUS_ERROR;
		}

		DVPGpuExternalResourceDesc Desc;
		Desc.width = (uint32) InArgs.Width;
		Desc.height = (uint32) InArgs.Height;
		Desc.size = InArgs.Stride * Desc.height;
		Desc.format = InArgs.PixelFormat == EPixelFormat::PF_8Bit ? DVP_BGRA : DVP_RGBA_INTEGER;
		Desc.type = InArgs.PixelFormat == EPixelFormat::PF_8Bit ? DVP_UNSIGNED_BYTE : DVP_INT;
		Desc.handleType = DVP_OPAQUE_WIN32;
		Desc.external.handle = OutTextureInfo->External.Handle;

		return dvpCreateGPUExternalResourceD3D12Device(D3DDevice, &Desc, &OutTextureInfo->DVPHandle);
	}

	DVPStatus FD3D12TextureTransfer::UnbindBuffer_Impl(DVPBufferHandle InBufferHandle) const
	{
		if (!D3DDevice)
		{
			return DVP_STATUS_ERROR;
		}

		return dvpUnbindFromD3D12Device(InBufferHandle, D3DDevice);
	}

	DVPStatus FD3D12TextureTransfer::MapBufferWaitAPI_Impl(DVPBufferHandle InHandle) const
	{
		if (!D3DDevice)
		{
			return DVP_STATUS_ERROR;
		}

		return (dvpMapBufferWaitD3D12(InHandle, D3DCommandQueue));
	}

	DVPStatus FD3D12TextureTransfer::MapBufferEndAPI_Impl(DVPBufferHandle InHandle) const
	{
		if (!D3DDevice)
		{
			return DVP_STATUS_ERROR;
		}

		return (dvpMapBufferEndD3D12(InHandle, D3DCommandQueue));
	}
}

#endif // DVP_SUPPORTED_PLATFORM
