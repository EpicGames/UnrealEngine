// Copyright Epic Games, Inc. All Rights Reserved.

#include "GPUTextureTransferModule.h"

#if DVP_SUPPORTED_PLATFORM
#include "D3D11TextureTransfer.h"
#include "D3D12TextureTransfer.h"
#include "RenderingThread.h"
#include "VulkanTextureTransfer.h"
#include "TextureTransferBase.h"
#endif

#if DVP_SUPPORTED_PLATFORM || PLATFORM_LINUX
#define VULKAN_PLATFORM 1
#else
#define VULKAN_PLATFORM 0
#endif

#if VULKAN_PLATFORM
#include "IVulkanDynamicRHI.h"
#endif

#include "CoreMinimal.h"
#include "GPUTextureTransfer.h"
#include "GenericPlatform/GenericPlatformDriver.h"
#include "HAL/PlatformMisc.h"
#include "Misc/App.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"

#include <unordered_map>

DEFINE_LOG_CATEGORY(LogGPUTextureTransfer);

static TAutoConsoleVariable<bool> CVarMediaIOEnableGPUDirect(
	TEXT("MediaIO.EnableGPUDirect"), false,
	TEXT("Whether to enable GPU direct for faster video frame copies. (Experimental)"),
	ECVF_RenderThreadSafe);

namespace 
{
	auto ConvertRHI = [](ERHIInterfaceType RHI)
	{
		switch (RHI)
		{
		case ERHIInterfaceType::D3D11: return UE::GPUTextureTransfer::ERHI::D3D11;
		case ERHIInterfaceType::D3D12: return UE::GPUTextureTransfer::ERHI::D3D12;
		case ERHIInterfaceType::Vulkan: return UE::GPUTextureTransfer::ERHI::Vulkan;
		default: return UE::GPUTextureTransfer::ERHI::Invalid;
		}
	};
}

bool FGPUTextureTransferModule::IsAvailable()
{
#if DVP_SUPPORTED_PLATFORM
	return FGPUTextureTransferModule::Get().IsInitialized() && FGPUTextureTransferModule::Get().IsEnabled();
#else
	return false;
#endif
}

void FGPUTextureTransferModule::StartupModule()
{
	if (CVarMediaIOEnableGPUDirect.GetValueOnAnyThread())
	{
		FCoreDelegates::OnAllModuleLoadingPhasesComplete.AddRaw(this, &FGPUTextureTransferModule::Initialize);
	}

	CVarMediaIOEnableGPUDirect.AsVariable()->OnChangedDelegate().AddRaw(this, &FGPUTextureTransferModule::OnEnableGPUDirectCVarChange);

	// Defer the following since the module is loaded too early for this call to succeed.
	FCoreDelegates::GetOnPostEngineInit().AddLambda([this]() 
	{
		// Cache this information since GetGPUDriverInfo has to be called on game thread because of a call to GetValueOnGameThread.
		CachedDriverInfo = FPlatformMisc::GetGPUDriverInfo(GRHIAdapterName);
	});
}

void FGPUTextureTransferModule::ShutdownModule()
{
	if (bInitialized)
	{
		UninitializeTextureTransfer();
	}
}

void FGPUTextureTransferModule::Initialize()
{
	if (FApp::CanEverRender())
	{
		if (LoadGPUDirectBinary())
		{
			TransferObjects.AddDefaulted(RHI_MAX);

			InitializeTextureTransfer();
		}
	}
}

UE::GPUTextureTransfer::TextureTransferPtr FGPUTextureTransferModule::GetTextureTransfer()
{
#if DVP_SUPPORTED_PLATFORM
	if (!IsInitialized())
	{
		UE_LOGF(LogGPUTextureTransfer, Warning, "GetTextureTransfer was called without initializing the library. This will cause a hitch since we have to block while waiting for the library to finish initializing.");
		Initialize();

		// Initialization is done on the rendering thread.
		FlushRenderingCommands();
	}
#endif


	if (FApp::CanEverRender())
	{
#if DVP_SUPPORTED_PLATFORM
		UE::GPUTextureTransfer::ERHI SupportedRHI = ConvertRHI(RHIGetInterfaceType());
		if (SupportedRHI == UE::GPUTextureTransfer::ERHI::Invalid) 
		{
			UE_LOGF(LogGPUTextureTransfer, Error, "The current RHI is not supported with GPU Texture Transfer.");
			return nullptr;
		}
	
		const uint8 RHIIndex = static_cast<uint8>(SupportedRHI);
		if (TransferObjects[RHIIndex])
		{
			return TransferObjects[RHIIndex];
		}
#endif
	}
	return nullptr;
}

bool FGPUTextureTransferModule::IsInitialized() const
{
#if DVP_SUPPORTED_PLATFORM
	return bInitialized;
#else
	return false;
#endif
}

bool FGPUTextureTransferModule::IsEnabled() const
{
	return CVarMediaIOEnableGPUDirect.GetValueOnAnyThread();
}

FGPUTextureTransferModule& FGPUTextureTransferModule::Get()
{
	return FModuleManager::LoadModuleChecked<FGPUTextureTransferModule>("GPUTextureTransfer");
}

bool FGPUTextureTransferModule::LoadGPUDirectBinary()
{
#if DVP_SUPPORTED_PLATFORM
	FString GPUDirectPath = FPaths::Combine(FPaths::EngineDir(), TEXT("Binaries/ThirdParty/NVIDIA/GPUDirect"), FPlatformProcess::GetBinariesSubdirectory());
	FPlatformProcess::PushDllDirectory(*GPUDirectPath);

	FString DVPDll;

	DVPDll = TEXT("dvp.dll");

	DVPDll = FPaths::Combine(GPUDirectPath, DVPDll);

	TextureTransferHandle = FPlatformProcess::GetDllHandle(*DVPDll);
	if (TextureTransferHandle == nullptr)
	{
		UE_LOGF(LogGPUTextureTransfer, Display, "Failed to load required library %ls. GPU Texture transfer will not be functional.", *DVPDll);
	}

	FPlatformProcess::PopDllDirectory(*GPUDirectPath);

#endif
	return !!TextureTransferHandle;
}


void FGPUTextureTransferModule::InitializeTextureTransfer()
{
#if DVP_SUPPORTED_PLATFORM

	bool bInitializationSuccess = true;

	static const TArray<FString> SupportedGPUPrefixes = {
		TEXT("RTX A4"),
		TEXT("RTX A5"),
		TEXT("RTX A6"),
		TEXT("Quadro")
	};

	bInitializationSuccess = CachedDriverInfo.IsNVIDIA() && !CachedDriverInfo.DeviceDescription.Contains(TEXT("Tesla"));

	const bool bRenderDocAttached = FParse::Param(FCommandLine::Get(), TEXT("AttachRenderDoc"));
	if (bRenderDocAttached)
	{
		bInitializationSuccess = false;
		// Render doc clashes with GPU Direct.
		UE_LOGF(LogGPUTextureTransfer, Display, "GPU Texture Transfer disabled because render is attached.")
	}

	if (bInitializationSuccess)
	{
		bInitializationSuccess = false;
		for (const FString& GPUPrefix : SupportedGPUPrefixes)
		{
			if (CachedDriverInfo.DeviceDescription.Contains(GPUPrefix))
			{
				bInitializationSuccess = true;
				break;
			}
		}
	}
	if (!bInitializationSuccess)
	{
		return;
	}

	if (!GDynamicRHI)
	{
		return;
	}

	ENQUEUE_RENDER_COMMAND(InitializeTextureTransfer)([this](FRHICommandListImmediate&)
	{
		UE::GPUTextureTransfer::TextureTransferPtr TextureTransfer;

		UE::GPUTextureTransfer::ERHI RHI = ConvertRHI(RHIGetInterfaceType());

		switch (RHI)
		{
		case UE::GPUTextureTransfer::ERHI::D3D11:
			TextureTransfer = MakeShared<UE::GPUTextureTransfer::Private::FD3D11TextureTransfer>();
			break;
		case UE::GPUTextureTransfer::ERHI::D3D12:
			TextureTransfer = MakeShared<UE::GPUTextureTransfer::Private::FD3D12TextureTransfer>();
			break;
		case UE::GPUTextureTransfer::ERHI::Vulkan:
			TextureTransfer = MakeShared<UE::GPUTextureTransfer::Private::FVulkanTextureTransfer>();
			break;
		default:
			ensureAlways(false);
			break;
		}

		const uint8 RHIIndex = static_cast<uint8>(RHI);
		UE_LOGF(LogGPUTextureTransfer, Display, "Initializing GPU Texture transfer");
		if (TextureTransfer->Initialize())
		{
			TransferObjects[RHIIndex] = TextureTransfer;
		}

		bInitialized = true;
	});
#endif // DVP_SUPPORTED_PLATFORM
}

void FGPUTextureTransferModule::UninitializeTextureTransfer()
{
#if DVP_SUPPORTED_PLATFORM
	for (uint8 RhiIt = 1; RhiIt < RHI_MAX; RhiIt++)
	{
		if (const UE::GPUTextureTransfer::TextureTransferPtr& TextureTransfer = TransferObjects[RhiIt])
		{
			TextureTransfer->Uninitialize();
		}
	}
#endif
}

void FGPUTextureTransferModule::OnEnableGPUDirectCVarChange(IConsoleVariable* ConsoleVariable)
{
	if (ConsoleVariable->GetBool() && !IsInitialized())
	{
		Initialize();
	}
}

IMPLEMENT_MODULE(FGPUTextureTransferModule, GPUTextureTransfer);
