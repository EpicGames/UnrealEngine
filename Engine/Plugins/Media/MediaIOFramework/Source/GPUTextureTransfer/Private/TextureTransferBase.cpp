// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextureTransferBase.h"
#include "GPUTextureTransferModule.h"
#include "Misc/ScopeLock.h"

#if DVP_SUPPORTED_PLATFORM
#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/HideWindowsPlatformTypes.h"


#if PERF_LOGGING
#include "ProfilingDebugging/ScopedTimers.h"
#endif


#if PERF_LOGGING
#define LOG_PERF(FuncName)\
FAutoScopedDurationTimer AutoTimer;\
ON_SCOPE_EXIT{ UE_LOGF(LogGPUTextureTransfer, Verbose, "%ls duration: %d ms", L#FuncName,	AutoTimer.GetTime()); }
#else
#define LOG_PERF(...)
#endif


#define DVP_CALL(call) \
{\
	DVPStatus _Status = (call); \
	if (_Status != DVP_STATUS_OK)\
	{\
		UE_LOGF(LogGPUTextureTransfer, Error, "GPUDirect call %ls failed. Error:  %d", L""#call, _Status);\
	}\
}

namespace UE::GPUTextureTransfer::Private
{
	FTextureTransferBase::DVPSync::DVPSync(uint32 SemaphoreAllocSize, uint32 SemaphoreAlignment)
	{
		// From GPU Direct documentation.
		Semaphore = (uint32*)FMemory::Malloc(SemaphoreAllocSize, SemaphoreAlignment);
		if (Semaphore)
		{
			*Semaphore = 0;
		}

		DVPSyncObjectDesc Description = { 0 };
		Description.sem = const_cast<uint32*>(Semaphore);

		DVP_CALL(dvpImportSyncObject(&Description, &DVPSyncObject));
	}

	FTextureTransferBase::DVPSync::~DVPSync()
	{
		DVP_CALL(dvpFreeSyncObject(DVPSyncObject));
		FMemory::Free((void*)Semaphore);
	}

	void FTextureTransferBase::DVPSync::SetValue(uint32 Value)
	{
		*(volatile uint32*)(Semaphore) = Value;
	}

	bool FTextureTransferBase::Initialize()
	{
		FScopeLock ScopeLock{ &CriticalSection };

		if (bInitialized)
		{
			return false;
		}

		if (Init_Impl() != DVP_STATUS_OK)
		{
			UE_LOGF(LogGPUTextureTransfer, Display, "GPU Direct failed to initialize.");
			return false;
		}

		DVP_CALL(GetConstants_Impl(&BufferAddressAlignment, &BufferGpuStrideAlignment, &SemaphoreAddressAlignment, &SemaphoreAllocSize,
			&SemaphorePayloadOffset, &SemaphorePayloadSize));

		bInitialized = true;
		return true;
	}

	bool FTextureTransferBase::Uninitialize()
	{
		FScopeLock ScopeLock{ &CriticalSection };
		if (!bInitialized)
		{
			return false;
		}

		ClearRegisteredTextures();
		ClearRegisteredBuffers();

		CloseDevice_Impl();

		bInitialized = false;
		return true;
	}

	void FTextureTransferBase::WaitForGPU(FRHITexture* InRHITexture)
	{
		FTextureInfo* TextureInfo = nullptr;
		{
			FScopeLock ScopeLock{ &CriticalSection };
			TextureInfo = RegisteredTextures.Find(InRHITexture);	
		}

		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("GPUDirect::WaitForGPU (%" UINT64_FMT ")"), TextureInfo->DVPHandle));
		DVP_CALL(dvpBegin());
		DVP_CALL(dvpMapBufferWaitDVP(TextureInfo->DVPHandle));
		DVP_CALL(dvpEnd());
	}

	void FTextureTransferBase::ThreadPrep()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(dvpBegin);
		DVP_CALL(dvpBegin());
	}

	void FTextureTransferBase::ThreadCleanup()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(dvpEnd);
		DVP_CALL(dvpEnd());
	}

	bool FTextureTransferBase::BeginSync(void* InBuffer, ETransferDirection TransferDirection)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(BeginSync);
		bool bSuccess = true;
		LOG_PERF(BeginSync);
		
		FExternalBufferInfo* Info = nullptr;
		{
			FScopeLock ScopeLock{ &CriticalSection };
			Info = RegisteredBuffers.Find(InBuffer);
		}

		if (ensureAlways(Info))
		{
			if (Info->GPUMemorySync && Info->SystemMemorySync)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("BeginSync Texture: %" UINT64_FMT ", Buffer %" UINT64_FMT), Info->PendingTextureHandle, Info->DVPHandle));
				Info->GPUMemorySync->AcquireValue++;

				Info->SystemMemorySync->ReleaseValue++;

				if (TransferDirection == ETransferDirection::GPU_TO_CPU)
				{
					constexpr uint64 NanosecondsToWait = 500000000; // 0.5 seconds
					DVPStatus SyncStatus = dvpSyncObjClientWaitPartial(Info->GPUMemorySync->DVPSyncObject, Info->GPUMemorySync->AcquireValue, NanosecondsToWait);
					if (SyncStatus != DVP_STATUS_OK)
					{
						bSuccess = false;
						UE_LOGF(LogGPUTextureTransfer, Error, "GPU Direct failed to sync.");
					}
				}
			}
			else
			{
				bSuccess = false;
				UE_LOGF(LogGPUTextureTransfer, Error, "Sync info was cleared prematurely while performing a GPU DMA Transfer sync");
			}
		}
		
		return bSuccess;

	}

	void FTextureTransferBase::EndSync(void* InBuffer)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(EndSync);

		FScopeLock ScopeLock{ &CriticalSection };
		if (FExternalBufferInfo* Info = RegisteredBuffers.Find(InBuffer))
		{
			if (Info->SystemMemorySync)
			{
				Info->SystemMemorySync->SetValue(Info->SystemMemorySync->ReleaseValue);
			}

			Info->PendingTextureHandle = 0;
		}
	}

	bool FTextureTransferBase::TransferTexture(void* InBuffer, FRHITexture* InRHITexture, ETransferDirection TransferDirection)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(TransferTexture);

		FScopeLock ScopeLock{ &CriticalSection };

		FExternalBufferInfo* BufferInfo = RegisteredBuffers.Find(InBuffer);
		FTextureInfo* TextureInfo = RegisteredTextures.Find(InRHITexture);

		if (!BufferInfo)
		{
			UE_LOG(LogGPUTextureTransfer, Error, TEXT("Error while performing a GPU transfer texture, CPU Buffer %" UPTRINT_FMT " was not registered."), reinterpret_cast<uintptr_t>(InBuffer));
			return false;
		}

		if (!TextureInfo)
		{
			UE_LOG(LogGPUTextureTransfer, Error, TEXT("Error while performing a GPU transfer texture, texture %" UPTRINT_FMT " was not registered."), reinterpret_cast<uintptr_t>(InRHITexture));
			return false;
		}

		if (BufferInfo->GPUMemorySync && BufferInfo->SystemMemorySync)
		{
			//UE_LOG(LogGPUTextureTransfer, Verbose, TEXT("Transferring texture Buffer: %d")

			BufferInfo->GPUMemorySync->ReleaseValue++;
			BufferInfo->PendingTextureHandle = TextureInfo->DVPHandle;

			DVPStatus Status = DVP_STATUS_OK;

			{
				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("dvpMemcpy2D (%" UINT64_FMT ")"), TextureInfo->DVPHandle));

				if (TransferDirection == ETransferDirection::GPU_TO_CPU)
				{
					Status = dvpMemcpy2D(TextureInfo->DVPHandle, BufferInfo->SystemMemorySync->DVPSyncObject,
						BufferInfo->SystemMemorySync->AcquireValue, DVP_TIMEOUT_IGNORED, BufferInfo->DVPHandle,
						BufferInfo->GPUMemorySync->DVPSyncObject, BufferInfo->GPUMemorySync->ReleaseValue, 0, 0, BufferInfo->Height, BufferInfo->Width);
				}
				else
				{
					Status = dvpMemcpy2D(BufferInfo->DVPHandle, BufferInfo->SystemMemorySync->DVPSyncObject,
						BufferInfo->SystemMemorySync->AcquireValue, DVP_TIMEOUT_IGNORED, TextureInfo->DVPHandle,
						BufferInfo->GPUMemorySync->DVPSyncObject, BufferInfo->GPUMemorySync->ReleaseValue, 0, 0, BufferInfo->Height, BufferInfo->Width);
				}
			}

			BufferInfo->SystemMemorySync->AcquireValue++;

			{
				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("dvpMapBufferEndDVP (%" UINT64_FMT ")"), TextureInfo->DVPHandle));
				DVP_CALL(dvpMapBufferEndDVP(TextureInfo->DVPHandle));
			}

			if (Status != DVP_STATUS_OK)
			{
				UE_LOGF(LogGPUTextureTransfer, Error, "Error while performing a GPU transfer texture. Error: '%d'.", Status);
				return false;

			}
			return true;
		}
		else
		{
			UE_LOGF(LogGPUTextureTransfer, Warning, "Error while performing a GPU transfer texture: A sync object was not found.");
		}

		return false;
	}

	void FTextureTransferBase::RegisterBuffer(const FRegisterDMABufferArgs& Args)
	{
		FScopeLock ScopeLock{ &CriticalSection };
		if (!Args.Buffer)
		{
			return;
		}

		if (!RegisteredBuffers.Contains(Args.Buffer))
		{
			//GPUTextureTransferPlatform::LockMemory(Args.Buffer, Args.Stride * Args.Height);

			FExternalBufferInfo BufferInfo;
			BufferInfo.Width = Args.Width;
			BufferInfo.Stride = Args.Stride;
			BufferInfo.Height = Args.Height;
			BufferInfo.SystemMemorySync = MakeUnique<DVPSync>(SemaphoreAllocSize, SemaphoreAddressAlignment);
			BufferInfo.GPUMemorySync = MakeUnique<DVPSync>(SemaphoreAllocSize, SemaphoreAddressAlignment);

			// todo jroy: Add conversion method from unreal pixel format to dvp pixel format to support more format/type combinations.

			// Register system memory buffers with DVP
			DVPSysmemBufferDesc SystemMemoryBuffersDesc;
			SystemMemoryBuffersDesc.width = Args.Width;
			SystemMemoryBuffersDesc.height = Args.Height;
			SystemMemoryBuffersDesc.stride = Args.Stride;
			SystemMemoryBuffersDesc.size = 0; // Only needed with DVP_BUFFER
			SystemMemoryBuffersDesc.format = Args.PixelFormat == EPixelFormat::PF_8Bit ? DVP_BGRA : DVP_RGBA_INTEGER;
			SystemMemoryBuffersDesc.type = Args.PixelFormat == EPixelFormat::PF_8Bit ? DVP_UNSIGNED_BYTE : DVP_INT;
			SystemMemoryBuffersDesc.bufAddr = Args.Buffer;

			DVP_CALL(dvpCreateBuffer(&SystemMemoryBuffersDesc, &BufferInfo.DVPHandle));
			DVP_CALL(BindBuffer_Impl(BufferInfo.DVPHandle));

			RegisteredBuffers.Add({ Args.Buffer, MoveTemp(BufferInfo) });
		}
	}

	void FTextureTransferBase::UnregisterBuffer(void* InBuffer)
	{
		FScopeLock ScopeLock{ &CriticalSection };
		if (FExternalBufferInfo* BufferInfo = RegisteredBuffers.Find(InBuffer))
		{
			const uint32 BufferSize = BufferInfo->Height * BufferInfo->Stride;
			ClearBufferInfo(*BufferInfo);
			//GPUTextureTransferPlatform::UnlockMemory(InBuffer, BufferSize);
			RegisteredBuffers.Remove(InBuffer);
		}
	}

	void FTextureTransferBase::RegisterTexture(const FRegisterDMATextureArgs& Args)
	{
		FScopeLock ScopeLock{ &CriticalSection };

		if (!RegisteredTextures.Contains(Args.RHITexture))
		{
			FTextureInfo Info;
			TRACE_CPUPROFILER_EVENT_SCOPE(CreateGPUResource_Impl);
			DVP_CALL(CreateGPUResource_Impl(Args, &Info));
			RegisteredTextures.Add({ Args.RHITexture, std::move(Info) });
		}
	}

	void FTextureTransferBase::UnregisterTexture(FRHITexture* InRHITexture)
	{
		FScopeLock ScopeLock{ &CriticalSection };
		if (FTextureInfo* TextureInfo = RegisteredTextures.Find(InRHITexture))
		{
			DVP_CALL(dvpFreeBuffer(TextureInfo->DVPHandle));
			if (TextureInfo->External.Handle)
			{
#if DVP_SUPPORTED_PLATFORM
				::CloseHandle(TextureInfo->External.Handle);
#endif
			}
			RegisteredTextures.Remove(InRHITexture);
		}
	}

	void FTextureTransferBase::ClearRegisteredTextures()
	{
		for (auto It = RegisteredTextures.CreateConstIterator(); It; ++It)
		{
			if (It.Value().DVPHandle)
			{
				dvpFreeBuffer(It.Value().DVPHandle);
				if (It.Value().External.Handle)
				{
#if DVP_SUPPORTED_PLATFORM
					::CloseHandle(It.Value().External.Handle);
#endif
				}
			}
		}

		RegisteredTextures.Reset();
	}

	void FTextureTransferBase::ClearRegisteredBuffers()
	{
		for (TPair<void*, FExternalBufferInfo>& Pair : RegisteredBuffers)
		{
			const uint32 BufferSize = Pair.Value.Height * Pair.Value.Stride;
			//todo: Fix this : GPUTextureTransferPlatform::UnlockMemory(It->first, BufferSize)
			ClearBufferInfo(Pair.Value);
		}

		RegisteredBuffers.Reset();
	}

	void FTextureTransferBase::LockTexture(FRHITexture* InRHITexture)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(TEXT("LockTexture"));
		FScopeLock ScopeLock{ &CriticalSection };

		if (FTextureInfo* TextureInfo = RegisteredTextures.Find(InRHITexture))
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("MapBufferWaitAPI_Impl (%" UINT64_FMT ")"), TextureInfo->DVPHandle));
			DVP_CALL(MapBufferWaitAPI_Impl(TextureInfo->DVPHandle));
		}
	}

	void FTextureTransferBase::UnlockTexture(FRHITexture* InRHITexture)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(TEXT("UnlockTexture"));
		LOG_PERF(UnlockTexture);

		FScopeLock ScopeLock{ &CriticalSection };
		if (FTextureInfo* TextureInfo = RegisteredTextures.Find(InRHITexture))
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("MapBufferEndAPI_Impl (%" UINT64_FMT ")"), TextureInfo->DVPHandle));
			DVP_CALL(MapBufferEndAPI_Impl(TextureInfo->DVPHandle));
		}
	}

	uint32 FTextureTransferBase::GetBufferAlignment() const
	{
		return BufferAddressAlignment;
	}

	uint32 FTextureTransferBase::GetTextureStride() const
	{
		return BufferGpuStrideAlignment;
	}

	void FTextureTransferBase::ClearBufferInfo(FExternalBufferInfo& BufferInfo)
	{
		DVP_CALL(UnbindBuffer_Impl(BufferInfo.DVPHandle));
		DVP_CALL(dvpDestroyBuffer(BufferInfo.DVPHandle));
		BufferInfo.SystemMemorySync.Reset();
		BufferInfo.GPUMemorySync.Reset();
	}

	DVPStatus FTextureTransferBase::MapBufferWaitAPI_Impl(DVPBufferHandle Handle) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("dvpMapBufferWaitAPI (%" UINT64_FMT ")"), Handle));
		return dvpMapBufferWaitAPI(Handle);
	}

	DVPStatus FTextureTransferBase::MapBufferEndAPI_Impl(DVPBufferHandle Handle) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("dvpMapBufferEndAPI (%" UINT64_FMT ")"), Handle));
		return dvpMapBufferEndAPI(Handle);
	}

}
#endif // DVP_SUPPORTED_PLATFORM
