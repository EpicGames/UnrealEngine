// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GPUTextureTransfer.h"

#if DVP_SUPPORTED_PLATFORM
#include "Windows/AllowWindowsPlatformTypes.h"
#include "DVPAPI.h"
#include "Windows/HideWindowsPlatformTypes.h"

class FRHITexture;

namespace UE::GPUTextureTransfer::Private
{
	class FTextureTransferBase : public ITextureTransfer
	{
	public:
		virtual ~FTextureTransferBase() = default;

		//~ Begin ITextureTransfer interface
		virtual void WaitForGPU(FRHITexture* InRHITexture) override;
		virtual void ThreadPrep() override;
		virtual void ThreadCleanup() override;
		virtual bool BeginSync(void* InBuffer, ETransferDirection TransferDirection) override;
		virtual void EndSync(void* InBuffer) override;
		virtual bool TransferTexture(void* InBuffer, FRHITexture* InRHITexture, ETransferDirection TransferDirection) override;
		virtual void RegisterBuffer(const FRegisterDMABufferArgs& Args) override;
		virtual void UnregisterBuffer(void* InBuffer) override;
		virtual void RegisterTexture(const FRegisterDMATextureArgs& Args) override;
		virtual void UnregisterTexture(FRHITexture* InRHITexture) override;
		virtual uint32 GetBufferAlignment() const override;
		virtual uint32 GetTextureStride() const override;
		virtual void LockTexture(FRHITexture* InRHITexture) override;
		virtual void UnlockTexture(FRHITexture* InRHITexture) override;
		// End ITextureTransfer interface

		virtual bool Initialize() override;
		virtual bool Uninitialize() override;

	protected:

		struct DVPSync
		{
			DVPSync(uint32 SemaphoreAllocSize, uint32 SemaphoreAlignment);
			~DVPSync();

			volatile uint32* Semaphore = nullptr;
			volatile uint32 ReleaseValue = 0;
			volatile uint32 AcquireValue = 0;
			DVPSyncObjectHandle DVPSyncObject = 0;

			void SetValue(uint32 Value);
		};

		struct FExternalBufferInfo
		{
			uint32 Width = 0;
			uint32 Height = 0;
			uint32 Stride = 0;
			TUniquePtr<DVPSync> SystemMemorySync;
			TUniquePtr<DVPSync> GPUMemorySync;
			DVPBufferHandle DVPHandle = 0;

			// Handle to the texture currently being transferred to this buffer.
			DVPBufferHandle PendingTextureHandle = 0;
		};

		struct FTextureInfo
		{
			DVPBufferHandle DVPHandle = 0;

			union
			{
				void* Handle = nullptr;
				int Fd;
			} External; // D3D12 and Vulkan only
		};

	protected:
		virtual DVPStatus Init_Impl() = 0;
		virtual DVPStatus GetConstants_Impl(uint32* OutBufferAddrAlignment, uint32* OutBufferGPUStrideAlignment, uint32* OutSemaphoreAddrAlignment, uint32* OutSemaphoreAllocSize, uint32* OutSemaphorePayloadOffset, uint32* OutSemaphorePayloadSize) const = 0;
		virtual DVPStatus BindBuffer_Impl(DVPBufferHandle InBufferHandle) const = 0;
		virtual DVPStatus CreateGPUResource_Impl(const FRegisterDMATextureArgs& InArgs, FTextureInfo* OutTextureInfo) const = 0;
		virtual DVPStatus UnbindBuffer_Impl(DVPBufferHandle InBufferHandle) const = 0;
		virtual DVPStatus CloseDevice_Impl() const = 0;
		virtual DVPStatus MapBufferWaitAPI_Impl(DVPBufferHandle InHandle) const;
		virtual DVPStatus MapBufferEndAPI_Impl(DVPBufferHandle InHandle) const;

	private:
		void ClearBufferInfo(FExternalBufferInfo& BufferInfo);
		void ClearRegisteredTextures();
		void ClearRegisteredBuffers();
	private:

		bool bInitialized = false;
		TMap<void*, FExternalBufferInfo> RegisteredBuffers;
		TMap<FRHITexture*, FTextureInfo> RegisteredTextures;

		uint32 BufferAddressAlignment = 0;
		uint32 BufferGpuStrideAlignment = 0;
		uint32 SemaphoreAddressAlignment = 0;
		uint32 SemaphoreAllocSize = 0;
		uint32 SemaphorePayloadOffset = 0;
		uint32 SemaphorePayloadSize = 0;

		FCriticalSection CriticalSection;
	};
}
#endif // DVP_SUPPORTED_PLATFORM
