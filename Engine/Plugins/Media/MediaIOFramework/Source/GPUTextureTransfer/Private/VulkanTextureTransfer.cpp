// Copyright Epic Games, Inc. All Rights Reserved.

#include "VulkanTextureTransfer.h"

#if DVP_SUPPORTED_PLATFORM
#include "Windows/AllowWindowsPlatformTypes.h"

#include "dvpapi_vulkan.h"

#include "Windows/HideWindowsPlatformTypes.h"


namespace UE::GPUTextureTransfer::Private
{
	DVPStatus FVulkanTextureTransfer::Init_Impl()
	{
		IVulkanDynamicRHI* VulkanRHI = GetIVulkanDynamicRHI();

		VulkanDevice = VulkanRHI->RHIGetVkDevice();
		VulkanQueue = VulkanRHI->RHIGetGraphicsVkQueue();

		uint8 RHIDeviceUUID[16] {};
		FMemory::Memcpy(RHIDeviceUUID, VulkanRHI->RHIGetVulkanDeviceUUID(), 16);

		return dvpInitVkDevice(VulkanDevice, 0, RHIDeviceUUID);
	}

	DVPStatus FVulkanTextureTransfer::GetConstants_Impl(uint32* OutBufferAddrAlignment, uint32* OutBufferGPUStrideAlignment, uint32* OutSemaphoreAddrAlignment, uint32* OutSemaphoreAllocSize, uint32* OutSemaphorePayloadOffset, uint32* OutSemaphorePayloadSize) const
	{
		if (!VulkanDevice)
		{
			return DVP_STATUS_ERROR;
		}

		return dvpGetRequiredConstantsVkDevice(OutBufferAddrAlignment, OutBufferGPUStrideAlignment, OutSemaphoreAddrAlignment, OutSemaphoreAllocSize,
			OutSemaphorePayloadOffset, OutSemaphorePayloadSize, VulkanDevice);
	}

	DVPStatus FVulkanTextureTransfer::CloseDevice_Impl() const
	{
		if (!VulkanDevice)
		{
			return DVP_STATUS_ERROR;
		}

		return dvpCloseVkDevice(VulkanDevice);
	}

	DVPStatus FVulkanTextureTransfer::BindBuffer_Impl(DVPBufferHandle InBufferHandle) const
	{
		if (!VulkanDevice)
		{
			return DVP_STATUS_ERROR;
		}

		return dvpBindToVkDevice(InBufferHandle, VulkanDevice);
	}

	DVPStatus FVulkanTextureTransfer::CreateGPUResource_Impl(const FRegisterDMATextureArgs& InArgs, FTextureTransferBase::FTextureInfo* OutTextureInfo) const
	{
		if (!OutTextureInfo || !VulkanDevice)
		{
			return DVP_STATUS_ERROR;
		}

		IVulkanDynamicRHI* VulkanRHI = GetIVulkanDynamicRHI();
		const FVulkanRHIAllocationInfo TextureAllocationInfo = VulkanRHI->RHIGetAllocationInfo(InArgs.RHITexture);

		VkMemoryGetWin32HandleInfoKHR MemoryGetHandleInfoKHR = {};
		MemoryGetHandleInfoKHR.sType = VK_STRUCTURE_TYPE_MEMORY_GET_WIN32_HANDLE_INFO_KHR;
		MemoryGetHandleInfoKHR.pNext = NULL;
		MemoryGetHandleInfoKHR.memory = TextureAllocationInfo.Handle;
		MemoryGetHandleInfoKHR.handleType = VK_EXTERNAL_MEMORY_HANDLE_TYPE_OPAQUE_WIN32_BIT;

		PFN_vkGetMemoryWin32HandleKHR GetMemoryWin32HandleKHR = (PFN_vkGetMemoryWin32HandleKHR)VulkanRHI->RHIGetVkDeviceProcAddr("vkGetMemoryWin32HandleKHR");
		if (!GetMemoryWin32HandleKHR)
		{
			return DVP_STATUS_ERROR;
		}

		HANDLE Handle;
		VERIFYVULKANRESULT_EXTERNAL(GetMemoryWin32HandleKHR(VulkanDevice, &MemoryGetHandleInfoKHR, &Handle));//&OutTextureInfo->External.Handle));

		DVPGpuExternalResourceDesc Desc;
		Desc.width = InArgs.Width;
		Desc.height = InArgs.Height;
		Desc.size = InArgs.Stride * Desc.height;
		Desc.format = InArgs.PixelFormat == EPixelFormat::PF_8Bit ? DVP_BGRA : DVP_RGBA_INTEGER;
		Desc.type = InArgs.PixelFormat == EPixelFormat::PF_8Bit ? DVP_UNSIGNED_BYTE : DVP_INT;
		Desc.handleType = DVP_OPAQUE_WIN32;
		Desc.external.handle = OutTextureInfo->External.Handle;

		return dvpCreateGPUExternalResourceVkDevice(VulkanDevice, &Desc, &OutTextureInfo->DVPHandle);
	}

	DVPStatus FVulkanTextureTransfer::UnbindBuffer_Impl(DVPBufferHandle InBufferHandle) const
	{
		if (!VulkanDevice)
		{
			return DVP_STATUS_ERROR;
		}

		return dvpUnbindFromVkDevice(InBufferHandle, VulkanDevice);
	}

	DVPStatus FVulkanTextureTransfer::MapBufferWaitAPI_Impl(DVPBufferHandle InHandle) const
	{
		if (!VulkanQueue)
		{
			return DVP_STATUS_ERROR;
		}

		return (dvpMapBufferWaitVk(InHandle, VulkanQueue));
	}

	DVPStatus FVulkanTextureTransfer::MapBufferEndAPI_Impl(DVPBufferHandle InHandle) const
	{
		if (!VulkanQueue)
		{
			return DVP_STATUS_ERROR;
		}

		return (dvpMapBufferEndVk(InHandle, VulkanQueue));
	}
}

#endif // DVP_SUPPORTED_PLATFORM