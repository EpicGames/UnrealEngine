// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TextureTransferBase.h"

#if DVP_SUPPORTED_PLATFORM
#include "IVulkanDynamicRHI.h"

namespace UE::GPUTextureTransfer::Private
{
class FVulkanTextureTransfer : public FTextureTransferBase
{
protected:
	virtual DVPStatus Init_Impl() override;
	virtual DVPStatus GetConstants_Impl(uint32* OutBufferAddrAlignment, uint32* OutBufferGPUStrideAlignment, uint32* OutSemaphoreAddrAlignment, uint32* OutSemaphoreAllocSize, uint32* OutSemaphorePayloadOffset, uint32* OutSemaphorePayloadSize) const override;
	virtual DVPStatus BindBuffer_Impl(DVPBufferHandle InBufferHandle) const override;
	virtual DVPStatus CreateGPUResource_Impl(const FRegisterDMATextureArgs& InArgs, FTextureInfo* OutTextureInfo) const override;
	virtual DVPStatus UnbindBuffer_Impl(DVPBufferHandle InBufferHandle) const override;
	virtual DVPStatus CloseDevice_Impl() const override;
	virtual DVPStatus MapBufferWaitAPI_Impl(DVPBufferHandle InHandle) const;
	virtual DVPStatus MapBufferEndAPI_Impl(DVPBufferHandle InHandle) const;


private:
	VkInstance VulkanInstance = nullptr;
	VkDevice VulkanDevice = nullptr;
	VkQueue VulkanQueue = nullptr;
};
}

#endif