// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"
#include "RHIResources.h"

namespace UE::GPUTextureTransfer
{
	enum class ERHI : uint8
	{
		Invalid,
		D3D11,
		D3D12,
		Vulkan,
		RHI_MAX
	};

	using FLoggingCallbackPtr = void(*)(const TCHAR* Format, ...);

	enum class EPixelFormat : uint8_t
	{
		PF_8Bit,
		PF_10Bit
	};

	struct FRegisterDMABufferArgs
	{
		void* Buffer = nullptr;
		uint32 Width = 0;
		uint32 Height = 0;
		uint32 Stride = 0;
		EPixelFormat PixelFormat = EPixelFormat::PF_8Bit;
	};

	struct FRegisterDMATextureArgs
	{
		FRHITexture* RHITexture = nullptr;
		void* RHIResourceMemory = nullptr; // Vulkan only
		uint32 Width = 0;
		uint32 Height = 0;
		EPixelFormat PixelFormat = EPixelFormat::PF_8Bit;
		// Stride in bytes
		uint32 Stride = 0;
		// Only for VK
		void* SharedHandle = nullptr;
	};

	enum class ETransferDirection :uint8
	{
		GPU_TO_CPU,
		CPU_TO_GPU
	};

	class ITextureTransfer
	{
	public:
		virtual ~ITextureTransfer() = default;

		/** Wait for a signal that the texture is ready to use by DVP. (Experimental) */
		virtual void WaitForGPU(FRHITexture* InRHITexture) = 0;
		/** Must be called called before TransferTexture on the same thread. */
		virtual void ThreadPrep() = 0;
		/** Must be called called after TransferTexture on the same thread. */
		virtual void ThreadCleanup() = 0;

		virtual void RegisterBuffer(const FRegisterDMABufferArgs& Args) = 0;
		virtual void UnregisterBuffer(void* InBuffer) = 0;

		virtual void RegisterTexture(const FRegisterDMATextureArgs& Args) = 0;
		virtual void UnregisterTexture(FRHITexture* RHITexture) = 0;

		/**
		 * Calling this will prevent the DVP library from using the RHI texture passed as argument until Unlock is called.
		 */
		virtual void LockTexture(FRHITexture* RHITexture) = 0;

		/**
		 * Calling this will allow the DVP library to access the RHI texture passed as argument.
		 */
		virtual void UnlockTexture(FRHITexture* RHITexture) = 0;

		virtual bool BeginSync(void* InBuffer, ETransferDirection TransferDirection) = 0;
		virtual void EndSync(void* InBuffer) = 0;

		// Usage: 
		/**
		 *  RegisterBuffer(BufferArgs);
		 *	TransferTexture(InArgs.Buffer, RHITexture, InTransferDirection);
		 * 	BeginSync(InArgs.Buffer, InTransferDirection);
		 *  
		 *  Schedule output frame
		 * 
		 *  EndSync(Buffer);
		 * 
		 */
		virtual bool TransferTexture(void* InBuffer, FRHITexture* RHITexture, ETransferDirection TransferDirection) = 0;

		/**
		 * Get the recommended alignment for the cpu buffer.
		 **/
		virtual uint32 GetBufferAlignment() const = 0;

		/**
		 * Get the recommended stride for textures that will be  copied.
		 */
		virtual uint32 GetTextureStride() const = 0;

		virtual bool Initialize() = 0;
		virtual bool Uninitialize() = 0;
	};
}
