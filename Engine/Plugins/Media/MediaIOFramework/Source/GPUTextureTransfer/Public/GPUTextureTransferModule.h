// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformDriver.h"
#include "GPUTextureTransfer.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"

#include <atomic>

#define UE_API GPUTEXTURETRANSFER_API

DECLARE_LOG_CATEGORY_EXTERN(LogGPUTextureTransfer, Log, All);

namespace UE::GPUTextureTransfer
{
	using TextureTransferPtr = TSharedPtr<ITextureTransfer>;
}

class FGPUTextureTransferModule : public IModuleInterface
{
public:
	static UE_API FGPUTextureTransferModule& Get();

	UE_DEPRECATED(5.6, "Use IsInitialized() and IsEnabled() instead.")
	static UE_API bool IsAvailable();

	//~ Begin IModuleInterface
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;
	//~ End IModuleInterface

	/** Load the DVP DLL and intialize the API.Note: This is a blocking call and could take upwards of 2s. */
	UE_API void Initialize();

	/** Get a texture transfer object that acts as a layer above the DVP library. */
	UE_API UE::GPUTextureTransfer::TextureTransferPtr GetTextureTransfer();

	/** Returns whether the DVP library was loaded and initialized. If this returns false, GetTextureTransfer will trigger the library to initialize. */
	UE_API bool IsInitialized() const;

	/** Returns whether GPUDirect is enabled. This reflects the value of MediaIO.EnableGPUDirect. You should check IsAvailable to know if the library is initialized. */
	UE_API bool IsEnabled() const;

private:
	/** Load the DVP dll. */
	UE_API bool LoadGPUDirectBinary();

	/** Initialize the DVP library and creates the ITextureTransfer objects. */
	UE_API void InitializeTextureTransfer();

	/** Clean up and uninitializes the DVP library. */
	UE_API void UninitializeTextureTransfer();

	/** Initializes the DVP library when MediaIO.EnableGPUDirect is set to 1. */
	UE_API void OnEnableGPUDirectCVarChange(class IConsoleVariable* ConsoleVariable);
	
private:
	/** Max number of RHIs that GPUTextureTransfer supports. */
	static constexpr uint8 RHI_MAX = static_cast<uint8>(UE::GPUTextureTransfer::ERHI::RHI_MAX);

	/** DVP DLL Handle. */
	void* TextureTransferHandle = nullptr;

	/** Tracks whether the dvp library and texture transfer objects were successfully created. */
	std::atomic<bool> bInitialized = false;
	
	/** Texture transfer objects. */
	TArray<UE::GPUTextureTransfer::TextureTransferPtr> TransferObjects;

	/** Cached information about the GPU Driver. */
	FGPUDriverInfo CachedDriverInfo;
};

#undef UE_API
