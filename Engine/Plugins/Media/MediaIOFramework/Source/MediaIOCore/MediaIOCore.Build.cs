// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class MediaIOCore : ModuleRules
	{
		public MediaIOCore(ReadOnlyTargetRules Target) : base(Target)
		{

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"AudioExtensions",
					"Core",
					"CoreUObject",
					"Engine",
					"ImageWriteQueue",
					"Media",
					"MediaAssets",
					"MediaUtils",
					"MovieSceneCapture",
					"OpenColorIO",
					"Projects",
					"RenderCore",
					"RHI",
					"Slate",
					"SlateCore",
					"TimeManagement"
				});

			PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "AudioMixer",
	                "AudioMixerCore",
					"GPUTextureTransfer",
					"ImageWrapper",
					"Renderer",
                    "SignalProcessing", 
	                "SoundFieldRendering"
				});

			PublicDefinitions.Add("WITH_MEDIA_IO_AUDIO_DEBUGGING=0");

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.Add("EditorFramework");
				PrivateDependencyModuleNames.Add("UnrealEd");
				PrivateDependencyModuleNames.Add("LevelEditor");
			}
		}
	}
}
