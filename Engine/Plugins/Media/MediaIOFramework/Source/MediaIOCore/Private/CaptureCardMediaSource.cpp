// Copyright Epic Games, Inc. All Rights Reserved.

#include "CaptureCardMediaSource.h"

#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CaptureCardMediaSource)


UCaptureCardMediaSource::UCaptureCardMediaSource()
	: NativeSourceColorSettings(MakeShared<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe>())
{
	Deinterlacer = CreateDefaultSubobject<UBobDeinterlacer>("Deinterlacer");
}

int64 UCaptureCardMediaSource::GetMediaOption(const FName& Key, int64 DefaultValue) const
{
	if (Key == UE::CaptureCardMediaSource::InterlaceFieldOrder)
	{
		return static_cast<int64>(InterlaceFieldOrder);
	}

	if (Key == UE::CaptureCardMediaSource::EvaluationType)
	{
		return (int64) EvaluationType;
	}

	return UTimeSynchronizableMediaSource::GetMediaOption(Key, DefaultValue);
}

TSharedPtr<IMediaOptions::FDataContainer, ESPMode::ThreadSafe> UCaptureCardMediaSource::GetMediaOption(const FName& Key, const TSharedPtr<FDataContainer, ESPMode::ThreadSafe>& DefaultValue) const
{
	if (Key ==  UE::CaptureCardMediaSource::OpenColorIOSettings)
	{
		FOpenColorIODataContainer Container;
		Container.ColorConversionSettings = ColorConversionSettings;
		return MakeShared<FOpenColorIODataContainer>(MoveTemp(Container));
	}

	if (Key == UE::CaptureCardMediaSource::SourceColorSettingsOption)
	{
		NativeSourceColorSettings->Update(SourceColorSettings);
		return NativeSourceColorSettings;
	}

	return Super::GetMediaOption(Key, DefaultValue);
}

FString UCaptureCardMediaSource::GetMediaOption(const FName& Key, const FString& DefaultValue) const
{
	if (Key == UE::CaptureCardMediaSource::Deinterlacer)
	{
		if (Deinterlacer)
		{
			return Deinterlacer->GetPathName();
		}
	}

	return UTimeSynchronizableMediaSource::GetMediaOption(Key, DefaultValue);
}

bool UCaptureCardMediaSource::GetMediaOption(const FName& Key, bool bDefaultValue) const
{
	if (Key == UE::CaptureCardMediaSource::RenderJIT)
	{
		return bRenderJIT;
	}

	if (Key == UE::CaptureCardMediaSource::Framelock)
	{
		return bFramelock;
	}

	return UTimeSynchronizableMediaSource::GetMediaOption(Key, bDefaultValue);
}

bool UCaptureCardMediaSource::HasMediaOption(const FName& Key) const
{
	if (Key == UE::CaptureCardMediaSource::Deinterlacer
		|| Key == UE::CaptureCardMediaSource::InterlaceFieldOrder
		|| Key == UE::CaptureCardMediaSource::SourceColorSettingsOption
		|| Key == UE::CaptureCardMediaSource::RenderJIT
		|| Key == UE::CaptureCardMediaSource::Framelock
		|| Key == UE::CaptureCardMediaSource::EvaluationType
		|| Key == UE::CaptureCardMediaSource::OpenColorIOSettings)
		{
			return true;
		}

	return UTimeSynchronizableMediaSource::HasMediaOption(Key);
}

void UCaptureCardMediaSource::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	
	Super::Serialize(Ar);
}

void UCaptureCardMediaSource::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::MediaColorSettingsUnification)
	{
		if (bOverrideSourceEncoding_DEPRECATED)
		{
			SourceColorSettings.EncodingOverride = static_cast<EMediaSourceEncoding>(OverrideSourceEncoding_DEPRECATED);
			bOverrideSourceEncoding_DEPRECATED = false;
		}

		if (bOverrideSourceColorSpace_DEPRECATED)
		{
			SourceColorSettings.ColorSpaceOverride = OverrideSourceColorSpace_DEPRECATED;
			bOverrideSourceColorSpace_DEPRECATED = false;
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
}

#if WITH_EDITOR
bool UCaptureCardMediaSource::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UCaptureCardMediaSource, bFramelock))
	{
		return bRenderJIT && EvaluationType == EMediaIOSampleEvaluationType::Timecode && bUseTimeSynchronization;
	}

	if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UCaptureCardMediaSource, SourceColorSettings))
	{
		// OpenColorIO takes priority over the regular source color setting overrides.
		return !ColorConversionSettings.IsValid();
	}

	return true;
}

void UCaptureCardMediaSource::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCaptureCardMediaSource, bUseTimeSynchronization))
	{
		if (!bUseTimeSynchronization)
		{
			bFramelock = false;
			EvaluationType = EMediaIOSampleEvaluationType::PlatformTime;
		}
	}

	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCaptureCardMediaSource, bRenderJIT))
	{
		if (!bRenderJIT)
		{
			bFramelock = false;
			EvaluationType = (EvaluationType == EMediaIOSampleEvaluationType::Latest ? EMediaIOSampleEvaluationType::PlatformTime : EvaluationType);
		}
	}

	if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCaptureCardMediaSource, EvaluationType))
	{
		if (EvaluationType != EMediaIOSampleEvaluationType::Timecode)
		{
			bFramelock = false;
		}
	}

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FMediaSourceColorSettings, ColorSpaceOverride))
	{
		SourceColorSettings.UpdateColorSpaceChromaticities();
	}

	const FName StructName = PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue()->GetFName();
	if (StructName == GET_MEMBER_NAME_CHECKED(ThisClass, SourceColorSettings))
	{
		NativeSourceColorSettings->Update(SourceColorSettings);
	}
}

void UCaptureCardMediaSource::SetLastDetectedConfiguration(const FMediaIOConfiguration& InConfiguration)
{
	check(IsInGameThread());
	LastDetectedConfiguration = InConfiguration;
	bHasLastDetectedConfiguration = true;
	LastDetectedConfigurationChanged.Broadcast();
}

void UCaptureCardMediaSource::ClearLastDetectedConfiguration()
{
	check(IsInGameThread());
	if (!bHasLastDetectedConfiguration)
	{
		return;
	}
	bHasLastDetectedConfiguration = false;
	LastDetectedConfiguration = FMediaIOConfiguration();
	LastDetectedConfigurationChanged.Broadcast();
}
#endif //WITH_EDITOR
