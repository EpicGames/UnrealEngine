// Copyright Epic Games, Inc. All Rights Reserved.

#include "FileMediaCapture.h"

#include "FileMediaOutput.h"
#include "IImageWrapperModule.h"
#include "ImageWriteQueue.h"
#include "ImageWriteTask.h"
#include "MediaIOCoreModule.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FileMediaCapture)

#if WITH_EDITOR
#include "AnalyticsEventAttribute.h"
#include "EngineAnalytics.h"
#endif

#if WITH_EDITOR
namespace FileMediaCaptureAnalytics
{
	/**
	 * @EventName MediaFramework.FileMediaCaptureStarted
	 * @Trigger Triggered when a file media capture of the viewport or render target is started.
	 * @Type Client
	 * @Owner MediaIO Team
	 */
	void SendCaptureEvent(const EImageFormat ImageFormat, int32 CompressionQuality, const FString& CaptureType)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			FString ImageFormatString;
			switch (ImageFormat)
			{
				case EImageFormat::PNG:
					ImageFormatString = TEXT("PNG");
					break;
				case EImageFormat::JPEG:
					ImageFormatString = TEXT("JPEG");
					break;
				case EImageFormat::GrayscaleJPEG:
					ImageFormatString = TEXT("GrayscaleJPEG");
					break;
				case EImageFormat::BMP:
					ImageFormatString = TEXT("BMP");
					break;
				case EImageFormat::ICO:
					ImageFormatString = TEXT("ICO");
					break;
				case EImageFormat::EXR:
					ImageFormatString = TEXT("EXR");
					break;
				case EImageFormat::ICNS:
					ImageFormatString = TEXT("ICNS");
					break;
				default:
					ImageFormatString = TEXT("Unknown");
					break;
			}

			TArray<FAnalyticsEventAttribute> EventAttributes;
			EventAttributes.Add(FAnalyticsEventAttribute(TEXT("CaptureType"), CaptureType));
			EventAttributes.Add(FAnalyticsEventAttribute(TEXT("ImageFormat"), MoveTemp(ImageFormatString)));
			EventAttributes.Add(FAnalyticsEventAttribute(TEXT("CompresionQuality"), FString::Printf(TEXT("%d"), CompressionQuality)));
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("MediaFramework.FileMediaCaptureStarted"), EventAttributes);
		}
	}
}
#endif

void UFileMediaCapture::OnFrameCaptured_RenderingThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, void* InBuffer, int32 Width, int32 Height, int32 BytesPerRow)
{
	IImageWriteQueueModule* ImageWriteQueueModule = FModuleManager::Get().GetModulePtr<IImageWriteQueueModule>("ImageWriteQueue");
	if (ImageWriteQueueModule == nullptr)
	{
		SetState(EMediaCaptureState::Error);
		return;
	}

	IImageWrapperModule* ImageModulePtr = FModuleManager::Get().GetModulePtr<IImageWrapperModule>("ImageWrapper");
	if (ImageModulePtr == nullptr)
	{
		SetState(EMediaCaptureState::Error);
		return;
	}

	TUniquePtr<FImageWriteTask> ImageTask = MakeUnique<FImageWriteTask>();
	ImageTask->Format = ImageFormat;
	ImageTask->Filename = FString::Printf(TEXT("%s%05lld.%s"), *BaseFilePathName, InBaseData.SourceFrameNumber, ImageModulePtr->GetExtension(ImageFormat));
	ImageTask->bOverwriteFile = bOverwriteFile;
	ImageTask->CompressionQuality = CompressionQuality;
	ImageTask->OnCompleted = OnCompleteWrapper;

	EPixelFormat PixelFormat = GetDesiredPixelFormat();
	if (PixelFormat == PF_B8G8R8A8)
	{
		// We only support tightly packed rows without padding
		if ((BytesPerRow != 0) && (BytesPerRow != (Width * 4)))
		{
			UE_LOGF(LogMediaIOCore, Error, "File media capture only supports tightly packed rows. Expected stride: %d. Received stride: %d. It might also mean that output resolution is too small.", Width * 4, BytesPerRow);
			SetState(EMediaCaptureState::Error);
			return;
		}

		TUniquePtr<TImagePixelData<FColor>> PixelData = MakeUnique<TImagePixelData<FColor>>(FIntPoint(Width, Height),
			TArray<FColor, FDefaultAllocator64>(reinterpret_cast<FColor*>(InBuffer), Width * Height));
		ImageTask->PixelData = MoveTemp(PixelData);
	}
	else if (PixelFormat == PF_FloatRGBA)
	{
		// We only support tightly packed rows without padding
		if ((BytesPerRow != 0) && (BytesPerRow != (Width * 8)))
		{
			UE_LOGF(LogMediaIOCore, Error, "File media capture only supports tightly packed rows. Expected stride: %d. Received stride: %d. It might also mean that output resolution is too small.", Width * 8, BytesPerRow);
			SetState(EMediaCaptureState::Error);
			return;
		}

		TUniquePtr<TImagePixelData<FFloat16Color>> PixelData = MakeUnique<TImagePixelData<FFloat16Color>>(FIntPoint(Width, Height), 
			TArray<FFloat16Color, FDefaultAllocator64>(reinterpret_cast<FFloat16Color*>(InBuffer), Width * Height));
		ImageTask->PixelData = MoveTemp(PixelData);
	}
	else
	{
		check(false);
	}

	TFuture<bool> DispatchedTask = ImageWriteQueueModule->GetWriteQueue().Enqueue(MoveTemp(ImageTask));

	if (!bAsync)
	{
		// If not async, wait for the dispatched task to complete.
		if (DispatchedTask.IsValid())
		{
			DispatchedTask.Wait();
		}
	}
}

bool UFileMediaCapture::InitializeCapture()
{
	FModuleManager::Get().LoadModuleChecked<IImageWriteQueueModule>("ImageWriteQueue");
	CacheMediaOutputValues();

	SetState(EMediaCaptureState::Capturing);

#if WITH_EDITOR
	FileMediaCaptureAnalytics::SendCaptureEvent(ImageFormat, CompressionQuality, GetCaptureSourceType());
#endif

	return true;
}

void UFileMediaCapture::CacheMediaOutputValues()
{
	UFileMediaOutput* FileMediaOutput = CastChecked<UFileMediaOutput>(MediaOutput);
	BaseFilePathName = FPaths::Combine(FileMediaOutput->FilePath.Path, FileMediaOutput->BaseFileName);
	ImageFormat = ImageFormatFromDesired(FileMediaOutput->WriteOptions.Format);
	CompressionQuality = FileMediaOutput->WriteOptions.CompressionQuality;
	bOverwriteFile = FileMediaOutput->WriteOptions.bOverwriteFile;
	bAsync = FileMediaOutput->WriteOptions.bAsync;

	OnCompleteWrapper = [NativeCB = FileMediaOutput->WriteOptions.NativeOnComplete, DynamicCB = FileMediaOutput->WriteOptions.OnComplete](bool bSuccess)
	{
		if (NativeCB)
		{
			NativeCB(bSuccess);
		}
		DynamicCB.ExecuteIfBound(bSuccess);
	};
}

