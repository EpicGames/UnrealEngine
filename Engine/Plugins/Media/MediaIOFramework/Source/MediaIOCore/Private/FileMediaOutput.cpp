// Copyright Epic Games, Inc. All Rights Reserved.

#include "FileMediaOutput.h"

#include "FileMediaCapture.h"
#include "Misc/Paths.h"
#include "UnrealEngine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FileMediaOutput)



UFileMediaOutput::UFileMediaOutput()
	: Super()
	, bInvertAlpha(true)
{
	FilePath.Path = FPaths::Combine(*FPaths::ProjectSavedDir(), *FString("MediaOutput"));
}


bool UFileMediaOutput::Validate(FString& OutFailureReason) const
{
	if (!Super::Validate(OutFailureReason))
	{
		return false;
	}

	if (FilePath.Path.IsEmpty())
	{
		OutFailureReason = FString::Printf(TEXT("Can't validate MediaOutput '%s'. The file path is null."), *GetName());
		return false;
	}

	if (GetRequestedPixelFormat() == PF_A2B10G10R10)
	{
		OutFailureReason = FString::Printf(TEXT("Can't validate MediaOutput '%s'. File media output doesn't support 10bits format."), *GetName());
		return false;
	}

	if (GetRequestedPixelFormat() != PF_B8G8R8A8 && WriteOptions.Format != EDesiredImageFormat::EXR)
	{
		OutFailureReason = FString::Printf(TEXT("Can't validate MediaOutput '%s'. Only EXR export is currently supported for PF_FloatRGBA and PF_A32B32G32R32F formats."), *GetName());
		return false;
	}

	return true;
}


FIntPoint UFileMediaOutput::GetRequestedSize() const
{
	if (bOverrideDesiredSize)
	{
		return DesiredSize;
	}

	return UMediaOutput::RequestCaptureSourceSize;
}


EPixelFormat UFileMediaOutput::GetRequestedPixelFormat() const
{
	if (bOverridePixelFormat)
	{
		return DesiredPixelFormat == EFileMediaOutputPixelFormat::FloatRGBA ? PF_FloatRGBA : PF_B8G8R8A8;
	}

	static const auto CVarDefaultBackBufferPixelFormat = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DefaultBackBufferPixelFormat"));
	EPixelFormat SceneTargetFormat = EDefaultBackBufferPixelFormat::Convert2PixelFormat(EDefaultBackBufferPixelFormat::FromInt(CVarDefaultBackBufferPixelFormat->GetValueOnAnyThread()));

	if (WriteOptions.Format == EDesiredImageFormat::EXR)
	{
		return SceneTargetFormat == EPixelFormat::PF_A2B10G10R10 ? PF_B8G8R8A8 : SceneTargetFormat;
	}
	else
	{
		return PF_B8G8R8A8;
	}
}


EMediaCaptureConversionOperation UFileMediaOutput::GetConversionOperation(EMediaCaptureSourceType InSourceType) const
{
	// All formats supporting alpha
	if (WriteOptions.Format == EDesiredImageFormat::EXR || WriteOptions.Format == EDesiredImageFormat::PNG)
	{
		// We invert alpha only when alpha channel as valid data when used with propagate alpha or using a render target, otherwise we force it to 1.0f.
		static const auto CVarPropagateAlpha = IConsoleManager::Get().FindConsoleVariable(TEXT("r.PostProcessing.PropagateAlpha"));
		const bool bPropagateAlpha = CVarPropagateAlpha->GetBool();
		if (bPropagateAlpha || (InSourceType == EMediaCaptureSourceType::RENDER_TARGET))
		{
			return bInvertAlpha ? EMediaCaptureConversionOperation::INVERT_ALPHA :
				EMediaCaptureConversionOperation::NONE;
		}
		else
		{
			return EMediaCaptureConversionOperation::SET_ALPHA_ONE;
		}
	}
	else
	{
		return EMediaCaptureConversionOperation::NONE;
	}
}

#if WITH_EDITOR
FString UFileMediaOutput::GetDescriptionString() const
{
	return FilePath.Path;
}

void UFileMediaOutput::GetDetailsPanelInfoElements(TArray<FInfoElement>& OutInfoElements) const
{
	OutInfoElements.Add(FInfoElement(
		NSLOCTEXT("FileMediaOutput", "MediaConfigurationHeader", "Media Configuration"),
		NSLOCTEXT("FileMediaOutput", "FilePathLabel", "File Path"),
		FText::FromString(FilePath.Path)));
}
#endif //WITH_EDITOR

UMediaCapture* UFileMediaOutput::CreateMediaCaptureImpl()
{
	UMediaCapture* Result = NewObject<UFileMediaCapture>();
	if (Result)
	{
		Result->SetMediaOutput(this);
	}
	return Result;
}

