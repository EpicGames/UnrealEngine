// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaCapture.h"

#include "Application/ThrottleManager.h"
#include "Async/Async.h"
#include "Engine/GameEngine.h"
#include "Engine/GameViewportClient.h"
#include "Engine/TextureRenderTarget2D.h"
#include "EngineModule.h"
#include "Engine/RendererSettings.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/ThreadManager.h"
#include "MediaCaptureHelper.h"
#include "MediaCaptureRenderPass.h"
#include "MediaCaptureSceneViewExtension.h"
#include "MediaCaptureSources.inl"
#include "MediaCaptureSyncPointWatcher.h"
#include "MediaIOFrameManager.h"
#include "MediaIOCoreModule.h"
#include "MediaOutput.h"
#include "MediaShaders.h"
#include "Misc/App.h"
#include "Misc/ScopeLock.h"
#include "OpenColorIODisplayExtension.h"
#include "RendererInterface.h"
#include "RenderGraphResources.h"
#include "ScreenPass.h"
#include "Slate/SlateViewportProvider.h"
#include "Slate/SceneViewport.h"
#include "TextureResource.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "RHIGPUReadback.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaCapture)

#if WITH_EDITOR
#include "AnalyticsEventAttribute.h"
#include "Editor/EditorEngine.h"
#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "EngineAnalytics.h"
#include "Framework/Notifications/NotificationManager.h"
#include "IAssetViewport.h"
#include "LevelEditor.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif //WITH_EDITOR

#define LOCTEXT_NAMESPACE "MediaCapture"

static TAutoConsoleVariable<int32> CVarMediaIOEnableExperimentalScheduling(
	TEXT("MediaIO.EnableExperimentalScheduling"), 1,
	TEXT("Whether to send out frame  in a separate thread. (Experimental)"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMediaIOScheduleOnAnyThread(
	TEXT("MediaIO.ScheduleOnAnyThread"), 1,
	TEXT("Whether to wait for resource readback in a separate thread. (Experimental)"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarMediaIOPrerollFrames(
	TEXT("MediaIO.PrerollFrames"), 0,
	TEXT("Whether to preroll frames during media capture. Enabling this may reduce latency by initializing GPU resources during the preroll phase"),
	ECVF_RenderThreadSafe);


static bool bSimulateError = false;

FAutoConsoleCommand FMediaIOSetErrorState
(
	TEXT("MediaIO.SimulateError"),
	TEXT("Causes the next call to ProcessCapture_RenderThread to fail."),
	FConsoleCommandDelegate::CreateLambda([]()
		{
			bSimulateError = true;
		})
);

DECLARE_GPU_STAT(MediaCapture_ProcessCapture);
DECLARE_CYCLE_STAT(TEXT("MediaCapture RenderThread LockResource"), STAT_MediaCapture_RenderThread_LockResource, STATGROUP_Media);
DECLARE_CYCLE_STAT(TEXT("MediaCapture RenderThread CPU Capture Callback"), STAT_MediaCapture_RenderThread_CaptureCallback, STATGROUP_Media);
DECLARE_CYCLE_STAT(TEXT("MediaCapture RenderThread RHI Capture Callback"), STAT_MediaCapture_RenderThread_RHI_CaptureCallback, STATGROUP_Media);

/* namespace MediaCaptureDetails definition
*****************************************************************************/

namespace MediaCaptureDetails
{
	bool FindSceneViewportAndLevel(TSharedPtr<FSceneViewport>& OutSceneViewport);

	//Validation that there is a capture
	bool ValidateIsCapturing(const UMediaCapture* CaptureToBeValidated);

	void ShowSlateNotification();

	static const FName LevelEditorName(TEXT("LevelEditor"));
}

#if WITH_EDITOR
namespace UE::MediaCaptureAnalytics
{
	/**
	 * @EventName MediaFramework.CaptureStarted
	 * @Trigger Triggered when a capture of the viewport or render target is started.
	 * @Type Client
	 * @Owner MediaIO Team
	 */
	void SendCaptureEvent(const FString& CaptureType)
	{
		if (FEngineAnalytics::IsAvailable())
		{
			TArray<FAnalyticsEventAttribute> EventAttributes;
			EventAttributes.Add(FAnalyticsEventAttribute(TEXT("CaptureType"), CaptureType));
			FEngineAnalytics::GetProvider().RecordEvent(TEXT("MediaFramework.CaptureStarted"), EventAttributes);
		}
	}
}
#endif


/* FMediaCaptureOptions
*****************************************************************************/
FMediaCaptureOptions::FMediaCaptureOptions()
	: Crop(EMediaCaptureCroppingType::None)
	, CustomCapturePoint(FIntPoint::ZeroValue)
	, bSkipFrameWhenRunningExpensiveTasks(true)
	, bConvertToDesiredPixelFormat(true)
	, bForceAlphaToOneOnConversion(false)
	, bAutostopOnCapture(false)
	, NumberOfFramesToCapture(-1)
{
}

void FMediaCaptureOptions::PostSerialize(const FArchive& Ar)
{
#if WITH_EDITOR
	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::MediaCaptureNewResizeMethods)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			ResizeMethod = bResizeSourceBuffer_DEPRECATED ? EMediaCaptureResizeMethod::ResizeSource : EMediaCaptureResizeMethod::ResizeInRenderPass;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
#endif
}

FString FMediaCaptureOptions::GetColorConversionSettingsString() const
{
	return ColorConversionSettings.ToString();
}

float FMediaCaptureTransform::GetRotationValue(bool bInDegrees) const
{
	float RotationValue;
	switch (Rotation)
	{
	case EMediaCaptureRotation::CW90:
		RotationValue = -90.0f;
		break;
	case EMediaCaptureRotation::CCW90:
		RotationValue = 90.0f;
		break;
	case EMediaCaptureRotation::CW180:
		RotationValue = 180.0f;
		break;
	case EMediaCaptureRotation::None:
	default: 
		RotationValue = 0.0f;
		break;
	}
	
	return bInDegrees ? RotationValue : FMath::DegreesToRadians(RotationValue);
}

FVector2f FMediaCaptureTransform::GetScaleValue() const
{
	FVector2f Scale = FVector2f::UnitVector;
	if (bFlipHorizontal)
	{
		Scale.X *= -1.0f;
	}
	if (bFlipVertical)
	{
		Scale.Y *= -1.0f;
	}
	
	return Scale;
}

/* UMediaCapture
*****************************************************************************/

UMediaCapture::UMediaCapture()
	: ValidSourceGPUMask(FRHIGPUMask::All())
	, bPendingOutputResourcesInitialized(false)
	, bShouldCaptureRHIResource(false)
	, WaitingForRenderCommandExecutionCounter(0)
	, PendingFrameCount(0)
	, bIsAutoRestartRequired(false)
{
}

UMediaCapture::~UMediaCapture() = default;

const TSet<EPixelFormat>& UMediaCapture::GetSupportedRgbaSwizzleFormats()
{
	static TSet<EPixelFormat> SupportedFormats =
		{
			PF_A32B32G32R32F,
			PF_B8G8R8A8,
			PF_G8,
			PF_G16,
			PF_FloatRGB,
			PF_FloatRGBA,
			PF_R32_FLOAT,
			PF_G16R16,
			PF_G16R16F,
			PF_G32R32F,
			PF_A2B10G10R10,
			PF_A16B16G16R16,
			PF_R16F,
			PF_FloatR11G11B10,
			PF_A8,
			PF_R32_UINT,
			PF_R32_SINT,
			PF_R16_UINT,
			PF_R16_SINT,
			PF_R16G16B16A16_UINT,
			PF_R16G16B16A16_SINT,
			PF_R5G6B5_UNORM,
			PF_R8G8B8A8,
			PF_A8R8G8B8,
			PF_R8G8,
			PF_R32G32B32A32_UINT,
			PF_R16G16_UINT,
			PF_R8_UINT,
			PF_R8G8B8A8_UINT,
			PF_R8G8B8A8_SNORM,
			PF_R16G16B16A16_UNORM,
			PF_R16G16B16A16_SNORM,
			PF_R32G32_UINT,
			PF_R8,
		};
	
	return SupportedFormats;
}

void UMediaCapture::BeginDestroy()
{
	if (GetState() == EMediaCaptureState::Capturing || GetState() == EMediaCaptureState::Preparing)
	{
		UE_LOGF(LogMediaIOCore, Warning, "%ls will be destroyed and the capture was not stopped.", *GetName());
	}
	StopCapture(false);

	Super::BeginDestroy();
}

FString UMediaCapture::GetDesc()
{
	if (MediaOutput)
	{
		return FString::Printf(TEXT("%s [%s]"), *Super::GetDesc(), *MediaOutput->GetDesc());
	}
	return FString::Printf(TEXT("%s [none]"), *Super::GetDesc());
}

bool UMediaCapture::CaptureActiveSceneViewport(FMediaCaptureOptions CaptureOptions)
{
	StopCapture(false);

	check(IsInGameThread());

	TSharedPtr<FSceneViewport> FoundSceneViewport;
	if (!MediaCaptureDetails::FindSceneViewportAndLevel(FoundSceneViewport) || !FoundSceneViewport.IsValid())
	{
		UE_LOGF(LogMediaIOCore, Warning, "Can not start the capture. No viewport could be found.");
		return false;
	}

	return CaptureSceneViewport(FoundSceneViewport, CaptureOptions);
}

bool UMediaCapture::CaptureSceneViewport(TSharedPtr<FSceneViewport>& SceneViewport, FMediaCaptureOptions CaptureOptions)
{
	using namespace UE::MediaCapture::Private;
	return StartSourceCapture(MakeShared<FSceneViewportCaptureSource>(this, CaptureOptions, SceneViewport));
}

bool UMediaCapture::CaptureRHITexture(const FRHICaptureResourceDescription& ResourceDescription, FMediaCaptureOptions CaptureOptions)
{
	using namespace UE::MediaCapture::Private;
	return StartSourceCapture(MakeShared<FRHIResourceCaptureSource>(this, CaptureOptions, ResourceDescription));
}

bool UMediaCapture::CaptureTextureRenderTarget2D(UTextureRenderTarget2D* RenderTarget, FMediaCaptureOptions CaptureOptions)
{
	using namespace UE::MediaCapture::Private;
	return StartSourceCapture(MakeShared<FRenderTargetCaptureSource>(this, CaptureOptions, RenderTarget));
}

bool UMediaCapture::CaptureMediaTexture(UMediaTexture* InMediaTexture, FMediaCaptureOptions CaptureOptions, float Rotation, const FVector2f& Scale)
{
	using namespace UE::MediaCapture::Private;
	return StartSourceCapture(MakeShared<FMediaTextureCaptureSource>(this, CaptureOptions, InMediaTexture, Rotation, Scale));
}

bool UMediaCapture::StartSourceCapture(TSharedPtr<UE::MediaCapture::Private::FCaptureSource> InSource)
{
	StopCapture(false);
	
	if (!InSource)
	{
		ensure(false);
		return false;
	}
	
	check(IsInGameThread());

	DesiredCaptureOptions = InSource->CaptureOptions;

	// BackBufferReady captures the final composited frame including UI, so alpha is not meaningful.
	// Force it to one to avoid outputting garbage alpha values.
	if (DesiredCaptureOptions.CapturePhase == EMediaCapturePhase::BackBufferReady)
	{
		DesiredCaptureOptions.bForceAlphaToOneOnConversion = true;
	}

	if (!ValidateMediaOutput())
	{
		MediaCaptureDetails::ShowSlateNotification();
		return false;
	}

	CacheMediaOutput(InSource->GetSourceType());

	if (bUseRequestedTargetSize)
	{
		DesiredSize = InSource->GetSize();
	}
	else if (DesiredCaptureOptions.ResizeMethod == EMediaCaptureResizeMethod::ResizeSource)
	{
		InSource->ResizeSourceBuffer(DesiredSize);
	}

	CacheOutputOptions();

	constexpr bool bCurrentlyCapturing = false;
	if (!InSource->ValidateSource(DesiredSize, DesiredPixelFormat, bCurrentlyCapturing))
	{
		MediaCaptureDetails::ShowSlateNotification();
		return false;
	}

	SetState(EMediaCaptureState::Preparing);

	bool bInitialized = InitializeCapture();
	if (bInitialized)
	{
		bInitialized = InSource->PostInitialize();
		
#if WITH_EDITOR
		if (ShouldAutoRecordCaptureStartedEvent())
		{
			RecordCaptureStartedEvent();
		}
#endif
	}

	// This could have been updated by the initialization done by the implementation
	bShouldCaptureRHIResource = ShouldCaptureRHIResource();

	CaptureRenderPipeline = MakePimpl<UE::MediaCapture::FRenderPipeline>(this);
	SyncPointWatcher = MakePimpl<UE::MediaCaptureData::FSyncPointWatcher>(this);

	if (bInitialized)
	{
		InitializeOutputResources(MediaOutput->NumberOfTextureBuffers);
		bInitialized = GetState() != EMediaCaptureState::Stopped && GetState() != EMediaCaptureState::Error;
	}

	if (bInitialized)
	{
		//no lock required the command on the render thread is not active yet
		CaptureSource = MoveTemp(InSource);
		CaptureSource->ApplyShowFlagChanges();

		CaptureSource->AddPasses(*CaptureRenderPipeline);
		
		if (DesiredCaptureOptions.CapturePhase != EMediaCapturePhase::EndFrame && DesiredCaptureOptions.CapturePhase != EMediaCapturePhase::BackBufferReady)
		{
			/** If we are running with color conversion, we need our extension to be executed before the OCIO extension is applied to the viewport. */
			const int32 Priority =  DesiredCaptureOptions.ColorConversionSettings.IsValid() ? OPENCOLORIO_SCENE_VIEW_EXTENSION_PRIORITY + 1 : 0;

			FRenderTarget* RenderTarget = nullptr;
			if (CaptureSource->GetSceneViewport())
			{
				RenderTarget = CaptureSource->GetSceneViewport().Get();
			}
			else if (CaptureSource->GetRenderTarget())
			{
				RenderTarget = CaptureSource->GetRenderTarget()->GameThread_GetRenderTargetResource();
			}

			if (RenderTarget) 
			{
				ViewExtension = FSceneViewExtensions::NewExtension<FMediaCaptureSceneViewExtension>(this, DesiredCaptureOptions.CapturePhase, Priority, RenderTarget);
			}
			else
			{
				UE_LOGF(LogMediaIOCore, Warning, "Could not acquire a render target to initialize the scene view extension. Consider changing the capture phase to End Frame.");
			}
		}

		if (ViewExtension)
		{
			ensure(CaptureSource->GetCaptureType() == UE::MediaCapture::Private::ECaptureType::Immediate);
		}

		if (CaptureSource->GetCaptureType() == UE::MediaCapture::Private::ECaptureType::Immediate)
		{
			if (DesiredCaptureOptions.CapturePhase == EMediaCapturePhase::BackBufferReady)
			{
				if (FSlateApplication::Get().GetRenderer())
				{
					FSlateApplication::Get().GetRenderer()->OnBackBufferReadyToPresent().AddUObject(this, &UMediaCapture::OnBackBufferReadyToPresent);
				}
			}

			// Immediate capture requires us to prepare frame info in OnBeginFrame
			FCoreDelegates::OnBeginFrame.AddUObject(this, &UMediaCapture::OnBeginFrame_GameThread);

			// Add frame info for current frame in case OnBeginFrame delegate has already fired
			AddFrameInfoForCaptureImmediate(GFrameCounter);
		}
		else
		{
			FCoreDelegates::OnEndFrame.AddUObject(this, &UMediaCapture::OnEndFrame_GameThread);
			UGameViewportClient::OnViewportRendered().AddUObject(this, &UMediaCapture::OnViewportRendered_GameThread);
		}
	}
	else
	{
		StopCapture(/*bAllowPendingFrameToBeProcess*/ false);
		MediaCaptureDetails::ShowSlateNotification();
	}

#if WITH_EDITOR
	UE::MediaCaptureAnalytics::SendCaptureEvent(GetCaptureSourceType());
#endif


	if (!GRHISupportsMultithreading)
	{
		UE_LOGF(LogMediaIOCore, Display, "Experimental scheduling and AnyThread Capture was disabled because the current RHI does not support Multithreading.");
	}
	else if (!SupportsAnyThreadCapture())
	{
		UE_LOGF(LogMediaIOCore, Display, "AnyThread Capture was disabled because the media capture implementation does not have a AnyThread callback or explicitly disabled AnyThread capture.");
	}

	return bInitialized;
}

void UMediaCapture::CacheMediaOutput(EMediaCaptureSourceType InSourceType)
{
	check(MediaOutput);
	CaptureSourceType = InSourceType;
	DesiredSize = MediaOutput->GetRequestedSize();
	bUseRequestedTargetSize = DesiredSize == UMediaOutput::RequestCaptureSourceSize;
	DesiredPixelFormat = MediaOutput->GetRequestedPixelFormat();
	ConversionOperation = MediaOutput->GetConversionOperation(InSourceType);
}

void UMediaCapture::CacheOutputOptions()
{
	DesiredOutputSize = GetOutputSize();
	DesiredOutputResourceType = GetOutputResourceType();
	DesiredOutputPixelFormat = GetOutputPixelFormat();
	DesiredOutputTextureDescription = GetOutputTextureDescription();
	DesiredOutputBufferDescription = GetOutputBufferDescription();
	MediaOutputName = *MediaOutput->GetName();
	bShouldCaptureRHIResource = ShouldCaptureRHIResource();
}

FIntPoint UMediaCapture::GetOutputSize() const
{
	switch (ConversionOperation)
	{
	case EMediaCaptureConversionOperation::RGBA8_TO_YUV_8BIT:
		return FIntPoint(DesiredSize.X / 2, DesiredSize.Y);
	case EMediaCaptureConversionOperation::RGB10_TO_YUVv210_10BIT:
		// Padding aligned on 48 (16 and 6 at the same time)
		return FIntPoint((((DesiredSize.X + 47) / 48) * 48) / 6, DesiredSize.Y);
	case EMediaCaptureConversionOperation::CUSTOM:
		return GetCustomOutputSize(DesiredSize);
	case EMediaCaptureConversionOperation::NONE:
	default:
		return DesiredSize;
	}
}

EPixelFormat UMediaCapture::GetOutputPixelFormat() const
{
	switch (ConversionOperation)
	{
	case EMediaCaptureConversionOperation::RGBA8_TO_YUV_8BIT:
		return EPixelFormat::PF_B8G8R8A8;
	case EMediaCaptureConversionOperation::RGB10_TO_YUVv210_10BIT:
		return EPixelFormat::PF_R32G32B32A32_UINT;
	case EMediaCaptureConversionOperation::CUSTOM:
		return GetCustomOutputPixelFormat(DesiredPixelFormat);
	case EMediaCaptureConversionOperation::NONE:
	default:
		return DesiredPixelFormat;
	}
}

EMediaCaptureResourceType UMediaCapture::GetOutputResourceType() const
{
	switch (ConversionOperation)
	{
	case EMediaCaptureConversionOperation::CUSTOM:
		return GetCustomOutputResourceType();
	default:
		return EMediaCaptureResourceType::Texture;
	}
}

FRDGBufferDesc UMediaCapture::GetOutputBufferDescription() const
{
	switch (ConversionOperation)
	{
	case EMediaCaptureConversionOperation::CUSTOM:
		return GetCustomBufferDescription(DesiredSize);
	default:
		return FRDGBufferDesc();
	}
}

FRDGTextureDesc UMediaCapture::GetOutputTextureDescription() const
{
	return FRDGTextureDesc::Create2D(
					GetOutputSize(),
					GetOutputPixelFormat(),
					FClearValueBinding::None,
					GetOutputTextureFlags());
}

bool UMediaCapture::UpdateSource(TSharedPtr<UE::MediaCapture::Private::FCaptureSource> InCaptureSource)
{
	if (!MediaCaptureDetails::ValidateIsCapturing(this))
	{
		StopCapture(false);
		return false;
	}

	check(IsInGameThread());

	const TSharedPtr<UE::MediaCapture::Private::FCaptureSource> PreviousCaptureSource = CaptureSource;
	CaptureSource = MoveTemp(InCaptureSource);

	if (!bUseRequestedTargetSize && DesiredCaptureOptions.ResizeMethod == EMediaCaptureResizeMethod::ResizeSource)
	{
		CaptureSource->ResizeSourceBuffer(DesiredSize);
	}

	const bool bCurrentlyCapturing = true;
	if (!CaptureSource->ValidateSource(DesiredSize, DesiredPixelFormat, bCurrentlyCapturing))
	{
		constexpr bool bFlushRenderingCommands = false;
		CaptureSource->ResetSourceBufferSize(bFlushRenderingCommands);
		StopCapture(false);
		MediaCaptureDetails::ShowSlateNotification();
		return false;
	}

	if (!CaptureSource->UpdateSourceImpl())
	{
		constexpr bool bFlushRenderingCommands = false;
		CaptureSource->ResetSourceBufferSize(bFlushRenderingCommands);
		StopCapture(false);
		MediaCaptureDetails::ShowSlateNotification();
		return false;
	}

	{
		FScopeLock Lock(&AccessingCapturingSource);

		WaitForPendingTasks();
		
		constexpr bool bFlushRenderingCommands = false;
		PreviousCaptureSource->ResetSourceBufferSize(bFlushRenderingCommands);
	}

	return true;
}

bool UMediaCapture::UseExperimentalScheduling() const
{
	return GRHISupportsMultithreading && CVarMediaIOEnableExperimentalScheduling.GetValueOnAnyThread() == 1;
}

#if WITH_EDITOR
void UMediaCapture::GetAnalyticsEventAttributes(const FString& EventName, TArray<FAnalyticsEventAttribute>& InOutAttributes) const
{
	const FIntPoint Resolution = GetDesiredSize();

	InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("CaptureType"), GetCaptureSourceType()));
	InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("ResolutionWidth"), FString::Printf(TEXT("%d"), Resolution.X)));
	InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("ResolutionHeight"), FString::Printf(TEXT("%d"), Resolution.Y)));
	InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesOCIO"), DesiredCaptureOptions.ColorConversionSettings.IsValid() ? TEXT("Yes") : TEXT("No")));
}

void UMediaCapture::RecordCaptureStartedEvent() const
{
	if (FEngineAnalytics::IsAvailable())
	{
		const FString EventName = TEXT("CaptureStarted");

		TArray<FAnalyticsEventAttribute> EventAttributes;
		GetAnalyticsEventAttributes(EventName, EventAttributes);

		FEngineAnalytics::GetProvider().RecordEvent(GetAnalyticsEventPrefix() + EventName, EventAttributes);
	}
}
#endif

bool UMediaCapture::UseAnyThreadCapture() const
{
	return GRHISupportsMultithreading
		&& CVarMediaIOEnableExperimentalScheduling.GetValueOnAnyThread()
		&& CVarMediaIOScheduleOnAnyThread.GetValueOnAnyThread()
		&& SupportsAnyThreadCapture();
}

bool UMediaCapture::UpdateSceneViewport(TSharedPtr<FSceneViewport>& InSceneViewport)
{
	check(CaptureSource);
	return UpdateSource(MakeShared<UE::MediaCapture::Private::FSceneViewportCaptureSource>(this, CaptureSource->CaptureOptions, InSceneViewport));
}

bool UMediaCapture::UpdateTextureRenderTarget2D(UTextureRenderTarget2D* InRenderTarget2D)
{
	check(CaptureSource);
	return UpdateSource(MakeShared<UE::MediaCapture::Private::FRenderTargetCaptureSource>(this, CaptureSource->CaptureOptions, InRenderTarget2D));
}

void UMediaCapture::StopCapture(bool bAllowPendingFrameToBeProcess)
{
	StopCapture_Callback(bAllowPendingFrameToBeProcess, FSimpleDelegate{});
}

void UMediaCapture::StopCapture_Callback(bool bAllowPendingFrameToBeProcess, FSimpleDelegate Callback)
{
	if (bWaitingForStopCapture)
	{
		// This capture is already in the process of stopping.
		return;
	}

	if (!IsInGameThread())
	{
		TWeakObjectPtr<UMediaCapture> Self = this;
		AsyncTask(ENamedThreads::GameThread, [Self, bAllowPendingFrameToBeProcess, Callback]()
			{
				UMediaCapture* MediaCapture = Self.Get();
				if (UObjectInitialized() && MediaCapture)
				{
					MediaCapture->StopCapture_Callback(bAllowPendingFrameToBeProcess, Callback);
				}
			});

		return;
	}

	if (GetState() != EMediaCaptureState::StopRequested && GetState() != EMediaCaptureState::Capturing)
	{
		bAllowPendingFrameToBeProcess = false;
	}

	if (bAllowPendingFrameToBeProcess)
	{
		if (GetState() != EMediaCaptureState::Stopped && GetState() != EMediaCaptureState::StopRequested)
		{
			SetState(EMediaCaptureState::StopRequested);

			//Do not flush when auto stopping to avoid hitches.
			if(DesiredCaptureOptions.bAutostopOnCapture != true)
			{
				WaitForPendingTasks();
			}
		}
	}
	else
	{
		if (FSlateApplication::IsInitialized())
		{
			if (FSlateApplication::Get().GetRenderer())
			{
				FSlateApplication::Get().GetRenderer()->OnBackBufferReadyToPresent().RemoveAll(this);
			}
		}

		if (FrameManager)
		{
			FrameManager->ForEachFrame([](const TSharedPtr<UE::MediaCaptureData::FFrame> InFrame)
			{
				StaticCastSharedPtr<UE::MediaCaptureData::FCaptureFrame>(InFrame)->bMediaCaptureActive = false;
			});
		}

		if (GetState() != EMediaCaptureState::Stopped)
		{
			if (SupportsStopCapture_Future())
			{
				SetState(EMediaCaptureState::StopRequested);
				// Prevent reentry and subsequent calls to StopCapture with bAllowPendingFrameToBeProcess = false to set the state to Stopped
				// while the TFuture isn't ready.
				bWaitingForStopCapture = true;
			}
			else
			{
				SetState(EMediaCaptureState::Stopped);
			}

			FCoreDelegates::OnBeginFrame.RemoveAll(this);
			FCoreDelegates::OnEndFrame.RemoveAll(this);
			UGameViewportClient::OnViewportRendered().RemoveAll(this);

			while (WaitingForRenderCommandExecutionCounter.load() > 0 || bPendingOutputResourcesInitialized)
			{
				FlushRenderingCommands();
			}
			
			if (SupportsStopCapture_Future())
			{
				TWeakObjectPtr<UMediaCapture> Self = this;

				TRACE_BOOKMARK(TEXT("UMediaCapture::InitiateStopCapture"));
				StopCapture_Future().Then([Self, Callback](TFuture<void>) mutable
				{
					AsyncTask(ENamedThreads::GameThread, [Self, Callback]
					{
						// Setting bEvenifPendingKill because this might be called as part of BeginDestroy,
						// but we still want to cleanup card connections.
						if (TStrongObjectPtr<UMediaCapture> Capture = Self.Pin(/*bEvenIfPendingKill=*/true)) 
						{
							Capture->SetState(EMediaCaptureState::Stopped);
							Capture->PostCaptureCleanup();
							Callback.ExecuteIfBound();
							TRACE_BOOKMARK(TEXT("UMediaCapture::CompleteStopCapture"));
						}
					});
				});
			}
			else
			{
				StopCaptureImpl(false);
				PostCaptureCleanup();
				Callback.ExecuteIfBound();
			}
		}
		else
		{
			Callback.ExecuteIfBound();
		}
	}
}

void UMediaCapture::SetMediaOutput(UMediaOutput* InMediaOutput)
{
	if (GetState() == EMediaCaptureState::Stopped)
	{
		MediaOutput = InMediaOutput;
	}
}

bool UMediaCapture::SetCaptureAudioDevice(const FAudioDeviceHandle& InAudioDeviceHandle)
{
	bool bSuccess = true;
	if (CaptureSource)
	{
		bSuccess = UpdateAudioDeviceImpl(InAudioDeviceHandle);
	}
	if (bSuccess)
	{
		AudioDeviceHandle = InAudioDeviceHandle;
	}
	return bSuccess;
}

void UMediaCapture::CaptureImmediate_RenderThread(FRDGBuilder& GraphBuilder, FRHITexture* InSourceTexture, FIntRect SourceViewRect)
{
	bool bCaptureSucceeded = TryCaptureImmediate_RenderThread(GraphBuilder, InSourceTexture, SourceViewRect);
	if (!bCaptureSucceeded)
	{
		UE_LOGF(LogMediaIOCore, VeryVerbose, "[%ls] - Failed to capture resource.", *MediaOutputName);
	}
}

bool UMediaCapture::TryCaptureImmediate_RenderThread(FRDGBuilder& GraphBuilder, FRHITexture* InSourceTexture, FIntRect SourceViewRect)
{
	UE::MediaCaptureData::FCaptureFrameArgs CaptureArgs{ GraphBuilder };
	CaptureArgs.MediaCapture = this;
	CaptureArgs.DesiredSize = DesiredSize;
	CaptureArgs.ResourceToCapture = InSourceTexture;
	CaptureArgs.SourceViewRect = SourceViewRect;

	return CaptureImmediate_RenderThread(CaptureArgs);
}

void UMediaCapture::CaptureImmediate_RenderThread(FRDGBuilder& GraphBuilder, FRDGTextureRef InSourceTextureRef, FIntRect SourceViewRect)
{
	bool bCaptureSucceeded = TryCaptureImmediate_RenderThread(GraphBuilder, InSourceTextureRef, SourceViewRect);
	if (!bCaptureSucceeded)
	{
		UE_LOGF(LogMediaIOCore, VeryVerbose, "[%ls] - Failed to capture resource.", *MediaOutputName);
	}
}

bool UMediaCapture::TryCaptureImmediate_RenderThread(FRDGBuilder& GraphBuilder, FRDGTextureRef InSourceTextureRef, FIntRect SourceViewRect)
{
	// Make sure captureimmediate runs are seen by game thread when stopping is requested. Having a pending render command will cause a flush
	// hence letting the capture finish. If stopping is in progress, stopped state will be seen by the captureimmediate and early exit.
	++WaitingForRenderCommandExecutionCounter;

	ON_SCOPE_EXIT
	{
		--WaitingForRenderCommandExecutionCounter;
	};

	UE::MediaCaptureData::FCaptureFrameArgs CaptureArgs{GraphBuilder};
	CaptureArgs.MediaCapture = this;
	CaptureArgs.DesiredSize = DesiredSize;
	CaptureArgs.RDGResourceToCapture = InSourceTextureRef;
	CaptureArgs.SourceViewRect = SourceViewRect;

	return CaptureImmediate_RenderThread(CaptureArgs);
}

bool UMediaCapture::CaptureImmediate_RenderThread(const UE::MediaCaptureData::FCaptureFrameArgs& Args)
{
	using namespace UE::MediaCaptureData;
	TRACE_CPUPROFILER_EVENT_SCOPE(UMediaCapture::CaptureImmediate_RenderThread);
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("MediaCapturePipe: %llu"), GFrameCounterRenderThread % 10));

	check(IsInRenderingThread());

	if (bIsAutoRestartRequired)
	{
		return false;
	}

	// This could happen if the capture immediate is called before our resources were initialized. We can't really flush as we are in a command.
	if (bPendingOutputResourcesInitialized)
	{
		UE_LOGF(LogMediaIOCore, VeryVerbose, "Could not capture frame. Output resources haven't been initialized");
		return false;
	}

	if (!MediaOutput)
	{
		return false;
	}

	if (GetState() == EMediaCaptureState::Error)
	{
		return false;
	}

	if (CaptureSource && CaptureSource->GetCaptureType() != UE::MediaCapture::Private::ECaptureType::Immediate)
	{
		UE_LOGF(LogMediaIOCore, Error, "[%ls] - Trying to capture a RHI resource with another capture type.", *MediaOutputName);
		SetState(EMediaCaptureState::Error);
	}

	if (GetState() != EMediaCaptureState::Capturing && GetState() != EMediaCaptureState::StopRequested)
	{
		return false;
	}

	if (DesiredCaptureOptions.bSkipFrameWhenRunningExpensiveTasks && !FSlateThrottleManager::Get().IsAllowingExpensiveTasks())
	{
		return false;
	}

	if (CaptureSource.IsValid() == false)
	{
		UE_LOGF(LogMediaIOCore, Error, "[%ls] - Trying to capture a RHI resource with an invalid source.", *MediaOutputName);
		SetState(EMediaCaptureState::Error);
		return false;
	}

	if (CaptureSource->GetSourceType() == EMediaCaptureSourceType::RHI_RESOURCE)
	{
		// Keep resource size up to date with incoming resource to capture
		StaticCastSharedPtr<UE::MediaCapture::Private::FRHIResourceCaptureSource>(CaptureSource)->ResourceDescription.ResourceSize = Args.GetSizeXY();
	}

	// Get cached capture data from game thread. We want to find a cached frame matching current render thread frame number
	bool bFoundMatchingData = false;
	FQueuedCaptureData NextCaptureData;

	{
		FScopeLock ScopeLock(&CaptureDataQueueCriticalSection);
		for (auto It = CaptureDataQueue.CreateIterator(); It; ++It)
		{
			if (It->BaseData.SourceFrameNumberRenderThread == GFrameCounterRenderThread)
			{
				NextCaptureData = *It;
				bFoundMatchingData = true;
				It.RemoveCurrent();
				break;
			}
			else if (GFrameCounterRenderThread > It->BaseData.SourceFrameNumberRenderThread &&
				GFrameCounterRenderThread - It->BaseData.SourceFrameNumberRenderThread > MaxCaptureDataAgeInFrames)
			{
				// Remove old frame data that wasn't used.
				It.RemoveCurrent();
			}
		}
	}
	if (bFoundMatchingData == false)
	{
		UE_LOGF(LogMediaIOCore, Warning, "Can't capture frame. Could not find the matching game frame %llu.", GFrameCounterRenderThread);
		return false;
	}

	if (GetState() == EMediaCaptureState::StopRequested && PendingFrameCount.load() <= 0)
	{
		// All the requested frames have been captured.
		StopCapture(false);
		return false;
	}

	// Get next available frame from the store. Can be invalid.
	TSharedPtr<FCaptureFrame> CapturingFrame;
	if (GetState() != EMediaCaptureState::StopRequested)
	{
		CapturingFrame = FrameManager->GetNextAvailable<FCaptureFrame>();
	}

	if (!CapturingFrame && GetState() != EMediaCaptureState::StopRequested)
	{
		if (DesiredCaptureOptions.OverrunAction == EMediaCaptureOverrunAction::Flush)
		{
			WaitForSingleExperimentalSchedulingTaskToComplete();

			CapturingFrame = FrameManager->GetNextAvailable<FCaptureFrame>();

			if (!CapturingFrame)
			{
				UE_LOGF(LogMediaIOCore, Error, "[%ls] - No frames available for capture for frame %llu. This should not happen.",
					*MediaOutputName, GFrameCounterRenderThread);

				SetState(EMediaCaptureState::Error);
				return false;
			}
		}
		else
		{
			//In case we are skipping frames, just keep capture frame as invalid
			UE_LOGF(LogMediaIOCore, Warning, "[%ls] - No frames available for capture of frame %llu. Skipping",
				*MediaOutputName, GFrameCounterRenderThread);
		}
	}

	PrintFrameState();

	if (!CapturingFrame || CapturingFrame->FrameId == -1)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UMediaCapture::Capture_Invalid);
	}

	UE_LOGF(LogMediaIOCore, Verbose, "[%ls - %ls] - Capturing frame %d", *MediaOutputName, *FThreadManager::GetThreadName(FPlatformTLS::GetCurrentThreadId()), CapturingFrame ? CapturingFrame->FrameId : -1);

	bool bCaptureSucceeded = false;
	if (CapturingFrame)
	{
		// Prepare frame to capture

		// Use queued capture base data for this frame to capture
		CapturingFrame->CaptureBaseData = MoveTemp(NextCaptureData.BaseData);
		CapturingFrame->CaptureBaseData.SourceFrameNumber = ++CaptureRequestCount;
		CapturingFrame->UserData = MoveTemp(NextCaptureData.UserData);

		bCaptureSucceeded = ProcessCapture_RenderThread(CapturingFrame, Args);
	}

	// If CVarMediaIOEnableExperimentalScheduling is enabled, it means that a sync point pass was added, and the ready frame processing will be done later
	if (!UseExperimentalScheduling())
	{
		if (TSharedPtr<FCaptureFrame> NextPending = FrameManager->PeekNextPending<FCaptureFrame>())
		{
			UE_LOGF(LogMediaIOCore, Verbose, "[%ls - %ls] - Processing pending frame %d", *MediaOutputName, *FThreadManager::GetThreadName(FPlatformTLS::GetCurrentThreadId()), NextPending ? NextPending->FrameId : -1);
			bCaptureSucceeded = ProcessReadyFrame_RenderThread(Args.GraphBuilder.RHICmdList, this, NextPending);
		}
	}

	return bCaptureSucceeded;
}

UTextureRenderTarget2D* UMediaCapture::GetTextureRenderTarget() const
{
	if (CaptureSource)
	{
		return CaptureSource->GetRenderTarget();
	}
	return nullptr;
}

TSharedPtr<FSceneViewport> UMediaCapture::GetCapturingSceneViewport() const
{
	if (CaptureSource)
	{
		return CaptureSource->GetSceneViewport();
	}
	return nullptr;
}

FString UMediaCapture::GetCaptureSourceType() const
{
	const UEnum* EnumType = StaticEnum<EMediaCaptureSourceType>();
	check(EnumType);
	return EnumType->GetNameStringByValue((int8)CaptureSourceType);
}

void UMediaCapture::SetState(EMediaCaptureState InNewState)
{
	if (MediaState != InNewState)
	{
		MediaState = InNewState;
		if (IsInGameThread())
		{
			BroadcastStateChanged();
		}
		else
		{
			TWeakObjectPtr<UMediaCapture> Self = this;
			AsyncTask(ENamedThreads::GameThread, [Self]
			{
				UMediaCapture* MediaCapture = Self.Get();
				if (UObjectInitialized() && MediaCapture)
				{
					MediaCapture->BroadcastStateChanged();
				}
			});
		}
	}
}

void UMediaCapture::RestartCapture()
{
	check(IsInGameThread());

	UE_LOGF(LogMediaIOCore, Log, "Media Capture restarting for new size %dx%d", CaptureSource->GetSize().X, CaptureSource->GetSize().Y);

	if (FrameManager)
	{
		FrameManager->ForEachFrame([](const TSharedPtr<UE::MediaCaptureData::FFrame> InFrame)
		{
			StaticCastSharedPtr<UE::MediaCaptureData::FCaptureFrame>(InFrame)->bMediaCaptureActive = false;
		});
	}

	constexpr bool bAllowPendingFrameToBeProcess = false;
	if (GetState() != EMediaCaptureState::Stopped)
	{
		SetState(EMediaCaptureState::Preparing);

		WaitForPendingTasks();

		StopCaptureImpl(bAllowPendingFrameToBeProcess);

		DesiredSize = CaptureSource->GetSize();
		CacheOutputOptions();

		bool bInitialized = InitializeCapture();
		if (bInitialized)
		{
			bInitialized = CaptureSource->PostInitialize();
		}

		// This could have been updated by the initialization done by the implementation
		bShouldCaptureRHIResource = ShouldCaptureRHIResource();

		if (bInitialized)
		{
			InitializeOutputResources(MediaOutput->NumberOfTextureBuffers);
			bInitialized = GetState() != EMediaCaptureState::Stopped;
		}

		if (!bInitialized)
		{
			SetState(EMediaCaptureState::Stopped);
			MediaCaptureDetails::ShowSlateNotification();
		}
	}

	bIsAutoRestartRequired = false;
}

void UMediaCapture::BroadcastStateChanged()
{
	OnStateChanged.Broadcast();
	OnStateChangedNative.Broadcast();
}

void UMediaCapture::SetFixedViewportSize(TSharedPtr<FSceneViewport> InSceneViewport, FIntPoint InSize)
{
	InSceneViewport->SetFixedViewportSize(InSize.X, InSize.Y);
	bViewportHasFixedViewportSize = true;
}

void UMediaCapture::ResetFixedViewportSize(TSharedPtr<FSceneViewport> InViewport, bool bInFlushRenderingCommands)
{
	if (bViewportHasFixedViewportSize && InViewport.IsValid())
	{
		if (bInFlushRenderingCommands)
		{
			WaitForPendingTasks();
		}
		InViewport->SetFixedViewportSize(0, 0);
		bViewportHasFixedViewportSize = false;
	}
}

const FMatrix& UMediaCapture::GetRGBToYUVConversionMatrix() const
{
	return MediaShaders::RgbToYuvRec709Scaled;
}

void UMediaCapture::WaitForSingleExperimentalSchedulingTaskToComplete()
{
	if (UseExperimentalScheduling())
	{
		// Presumably the rendering thread could be in the process of dispatching the task
		SyncPointWatcher->WaitForSinglePendingTaskToComplete();

		if (!UseAnyThreadCapture())
		{
			// We flush after task completion in case a render thread task was launched.
			if (IsInGameThread())
			{
				FlushRenderingCommands();
			}
		}
	}
}

void UMediaCapture::WaitForAllExperimentalSchedulingTasksToComplete()
{
	if (UseExperimentalScheduling())
	{
		SyncPointWatcher->WaitForAllPendingTasksToComplete();

		if (!UseAnyThreadCapture())
		{
			// This code might have dispatched a task on the render thread, so we need to wait again
			if (IsInGameThread())
			{
				FlushRenderingCommands();
			}
		}
	}
}

void UMediaCapture::WaitForPendingTasks()
{
	while (WaitingForRenderCommandExecutionCounter.load() > 0)
	{
		FlushRenderingCommands();
	}

	WaitForAllExperimentalSchedulingTasksToComplete();
}

void UMediaCapture::OnBackBufferReadyToPresent(SWindow& SlateWindow, ISlateViewportProvider& ViewportProvider)
{
	if (GetState() != EMediaCaptureState::Stopped)
	{
		if (CaptureSource->GetTargetWindow().Get() == &SlateWindow)
		{
			FRDGBuilder GraphBuilder(FRHICommandListImmediate::Get());
			TryCaptureImmediate_RenderThread(GraphBuilder, ViewportProvider.GetBackBufferResource());
			GraphBuilder.Execute();
		}
	}
}

void UMediaCapture::ProcessCapture_GameThread()
{
	using namespace UE::MediaCaptureData;

	// Acquire a frame
	TSharedPtr<FCaptureFrame> CapturingFrame;
	if(GetState() != EMediaCaptureState::StopRequested)
	{
		CapturingFrame = FrameManager->GetNextAvailable<FCaptureFrame>();
	}

	// Handle frame overrun (couldn't acquire a frame)
	if (!CapturingFrame && GetState() != EMediaCaptureState::StopRequested)
	{
		if (DesiredCaptureOptions.OverrunAction == EMediaCaptureOverrunAction::Flush)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MediaCapture::FlushRenderingCommands);
			UE_LOGF(LogMediaIOCore, VeryVerbose, "[%ls] - Flushing commands.", *MediaOutputName);

			FlushRenderingCommands();

			WaitForSingleExperimentalSchedulingTaskToComplete();

			// After flushing, we should have access to an available frame, if not, it's not expected.
			CapturingFrame = FrameManager->GetNextAvailable<FCaptureFrame>();

			if (!CapturingFrame)
			{
				UE_LOGF(LogMediaIOCore, Error, "[%ls] - Flushing commands didn't give us back available frames to process." , *MediaOutputName);
				StopCapture(false);
				return;
			}
		}
		else if (DesiredCaptureOptions.OverrunAction == EMediaCaptureOverrunAction::Skip)
		{
			// Selected options is to skip capturing a frame if overrun happens
			UE_LOGF(LogMediaIOCore, Verbose, "[%ls] - No frames available for capture. Skipping", *MediaOutputName);
		}
	}

	// Initialize capture frame
	InitializeCaptureFrame(CapturingFrame);

	PrintFrameState();
	
	PrepareAndDispatchCapture_GameThread(CapturingFrame);
}

void UMediaCapture::PrepareAndDispatchCapture_GameThread(const TSharedPtr<UE::MediaCaptureData::FCaptureFrame>& CapturingFrame)
{
	using namespace UE::MediaCaptureData;
	
	// Init variables for ENQUEUE_RENDER_COMMAND.
	//The Lock only synchronize while we are copying the value to the enqueue. The viewport and the rendertarget may change while we are in the enqueue command.
	{
		FScopeLock Lock(&AccessingCapturingSource);

		if (CaptureSource.IsValid())
		{
			++WaitingForRenderCommandExecutionCounter;

			const FRHIGPUMask SourceGPUMask = ValidSourceGPUMask;

			TWeakObjectPtr<UMediaCapture> WeakThis = this;

			UE_LOGF(LogMediaIOCore, Verbose, "[%ls] - Queuing frame to capture %d", *MediaOutputName, CapturingFrame ? CapturingFrame->FrameId : -1);

			// RenderCommand to be executed on the RenderThread
			ENQUEUE_RENDER_COMMAND(FMediaOutputCaptureFrameCreateTexture)(
				[WeakThis, CaptureSource_RT = CaptureSource, CapturingFrame, InDesiredSize = DesiredSize, SourceGPUMask](FRHICommandListImmediate& RHICmdList)
			{
				SCOPED_GPU_MASK(RHICmdList, SourceGPUMask);

				TStrongObjectPtr<UMediaCapture> This = WeakThis.Pin();
				if (!This.IsValid())
				{
					return;
				}

				TSharedPtr<FCaptureFrame> NextPending = This->FrameManager->PeekNextPending<FCaptureFrame>();
				if (NextPending && CapturingFrame)
				{
					ensure(NextPending->FrameId != CapturingFrame->FrameId);
				}
				
				// Capture frame
				{
					FRDGBuilder GraphBuilder(RHICmdList);
					
					FTextureRHIRef SourceTexture = CaptureSource_RT->GetSourceTextureForInput_RenderThread();
					
					FCaptureFrameArgs CaptureArgs{ GraphBuilder };
					CaptureArgs.MediaCapture = This.Get();
					CaptureArgs.DesiredSize = InDesiredSize;
					CaptureArgs.ResourceToCapture = MoveTemp(SourceTexture);

					This->ProcessCapture_RenderThread(CapturingFrame, CaptureArgs);
					UE_LOGF(LogMediaIOCore, Verbose, "[%ls] - Captured frame %d", *This->MediaOutputName, CapturingFrame ? CapturingFrame->FrameId : -1);
					
					GraphBuilder.Execute();
				}
				
				if (!This->UseExperimentalScheduling())
				{
					//Process the next pending frame
					UE_LOGF(LogMediaIOCore, Verbose, "[%ls] - Processing pending frame %d", *This->MediaOutputName, NextPending ? NextPending->FrameId : -1);
					if (NextPending)
					{
						This->ProcessReadyFrame_RenderThread(RHICmdList, This.Get(), NextPending);
					}
				}

				// Whatever happens, we want to decrement our counter to track enqueued commands
				--This->WaitingForRenderCommandExecutionCounter;
			});

			//If auto-stopping, count the number of frame captures requested and stop when reaching 0.
			if (DesiredCaptureOptions.bAutostopOnCapture && GetState() == EMediaCaptureState::Capturing && CaptureRequestCount == DesiredCaptureOptions.NumberOfFramesToCapture)
			{
				StopCapture(true);
			}
		}
	}
}

bool UMediaCapture::HasFinishedProcessing() const
{
	return WaitingForRenderCommandExecutionCounter.load() == 0
		|| GetState() == EMediaCaptureState::Error
		|| GetState() == EMediaCaptureState::Stopped;
}

void UMediaCapture::SetValidSourceGPUMask(FRHIGPUMask GPUMask)
{
	ValidSourceGPUMask = GPUMask;
}

void UMediaCapture::InitializeOutputResources(int32 InNumberOfBuffers)
{
	using namespace UE::MediaCaptureData;

	if (DesiredOutputSize.X <= 0 || DesiredOutputSize.Y <= 0)
	{
		UE_LOGF(LogMediaIOCore, Error, "Can't start the capture. The size requested is negative or zero.");
		SetState(EMediaCaptureState::Error);
		return;
	}

	// Recreate frame manager which can trigger cleaning up its captured frames if it exists
	FrameManager = MakePimpl<FFrameManager>();

	bPendingOutputResourcesInitialized = true;
	bSyncHandlersInitialized = false;
	NumberOfCaptureFrame = InNumberOfBuffers;

	TWeakObjectPtr<UMediaCapture> WeakThis = this;
	ENQUEUE_RENDER_COMMAND(MediaOutputCaptureFrameCreateResources)(
	[WeakThis](FRHICommandListImmediate& RHICmdList)
		{
			TStrongObjectPtr<UMediaCapture> This = WeakThis.Pin();
			if (!This.IsValid())
			{
				return;
			}

			for (int32 Index = 0; Index < This->NumberOfCaptureFrame; ++Index)
			{
				if (This->DesiredOutputResourceType == EMediaCaptureResourceType::Texture)
				{
					TSharedPtr<FTextureCaptureFrame> NewFrame = MakeShared<FTextureCaptureFrame>(Index);
					
					// Only create CPU readback resource when we are using the CPU callback
					if (!This->bShouldCaptureRHIResource)
					{
						NewFrame->ReadbackTexture = MakeUnique<FRHIGPUTextureReadback>(*FString::Printf(TEXT("MediaCaptureTextureReadback_%d"), Index));
					}

					if (This->GetDesiredCaptureOptions().bAutostopOnCapture || !CVarMediaIOPrerollFrames.GetValueOnRenderThread())
					{
						// Don't preroll if capturing a set number of frames since we discard the first frames.
						NewFrame->bPrerollFrame = false;
					}

					This->FrameManager->AddFrame(MoveTemp(NewFrame));
				}
				else
				{
					TSharedPtr<FBufferCaptureFrame> NewFrame = MakeShared<FBufferCaptureFrame>(Index);
					
					if (This->DesiredOutputBufferDescription.NumElements > 0)
					{
						// Only create CPU readback resource when we are using the CPU callback
						if (!This->bShouldCaptureRHIResource)
						{
							NewFrame->ReadbackBuffer = MakeUnique<FRHIGPUBufferReadback>(*FString::Printf(TEXT("MediaCaptureBufferReadback_%d"), Index));
						}

						if (This->GetDesiredCaptureOptions().bAutostopOnCapture)
						{
							// Don't preroll if capturing a set number of frames since we discard the first frames.
							NewFrame->bPrerollFrame = false;
						}

						This->FrameManager->AddFrame(MoveTemp(NewFrame));
					}
					else
					{
						UE_LOGF(LogMediaIOCore, Error, "Can't start the capture. Trying to allocate buffer resource but number of elements to allocate was 0.");
						This->SetState(EMediaCaptureState::Error);
					}
				}
			}
			This->bPendingOutputResourcesInitialized = false;
		});
}

void UMediaCapture::InitializeSyncHandlers_RenderThread()
{
	SyncHandlers.Reset(NumberOfCaptureFrame);
	for (int32 Index = 0; Index < NumberOfCaptureFrame; ++Index)
	{
		TSharedPtr<FMediaCaptureSyncData> SyncData = MakeShared<FMediaCaptureSyncData>();
		if (!ShouldCaptureRHIResource())
		{
			SyncData->RHIFence = RHICreateGPUFence(*FString::Printf(TEXT("MediaCaptureSync_%02d"), Index));
		}
		SyncHandlers.Add(MoveTemp(SyncData));
	}

	bSyncHandlersInitialized = true;
}

void UMediaCapture::InitializeCaptureFrame(const TSharedPtr<UE::MediaCaptureData::FCaptureFrame>& CaptureFrame)
{
	if (CaptureFrame)
	{
		CaptureFrame->CaptureBaseData.SourceFrameTimecode = FApp::GetTimecode();
		CaptureFrame->CaptureBaseData.SourceFrameTimecodeFramerate = FApp::GetTimecodeFrameRate();
		CaptureFrame->CaptureBaseData.SourceFrameNumberRenderThread = GFrameCounter;
		CaptureFrame->CaptureBaseData.SourceFrameNumber = ++CaptureRequestCount;
		CaptureFrame->UserData = GetCaptureFrameUserData_GameThread();
	}
}

TSharedPtr<UMediaCapture::FMediaCaptureSyncData> UMediaCapture::GetAvailableSyncHandler() const
{
	const auto FindAvailableHandlerFunc = [](const TSharedPtr<FMediaCaptureSyncData>& Item)
	{
		if (Item->bIsBusy == false)
		{
			return true;
		}

		return false;
	};

	if (const TSharedPtr<FMediaCaptureSyncData>* FoundItem = SyncHandlers.FindByPredicate(FindAvailableHandlerFunc))
	{
		return *FoundItem;
	}

	return nullptr;
}

bool UMediaCapture::ValidateMediaOutput() const
{
	if (MediaOutput == nullptr)
	{
		UE_LOGF(LogMediaIOCore, Error, "Can not start the capture. The Media Output is invalid.");
		return false;
	}

	FString FailureReason;
	if (!MediaOutput->Validate(FailureReason))
	{
		UE_LOGF(LogMediaIOCore, Error, "Can not start the capture. %ls.", *FailureReason);
		return false;
	}

	if(DesiredCaptureOptions.bAutostopOnCapture && DesiredCaptureOptions.NumberOfFramesToCapture < 1)
	{
		UE_LOGF(LogMediaIOCore, Error, "Can not start the capture. Please set the Number Of Frames To Capture when using Autostop On Capture in the Media Capture Options");
		return false;
	}

	return true;
}

bool UMediaCapture::CaptureSceneViewportImpl(TSharedPtr<FSceneViewport>&InSceneViewport)
{
	return false;
}

bool UMediaCapture::CaptureRenderTargetImpl(UTextureRenderTarget2D * InRenderTarget)
{
	return false;
}

void UMediaCapture::AddFrameInfoForCaptureImmediate(const uint64 FrameNumber)
{
	// Prepare capture data

	FQueuedCaptureData CaptureData;
	CaptureData.BaseData.SourceFrameTimecode = FApp::GetTimecode();
	CaptureData.BaseData.SourceFrameTimecodeFramerate = FApp::GetTimecodeFrameRate();
	CaptureData.BaseData.SourceFrameNumberRenderThread = FrameNumber;
	CaptureData.UserData = GetCaptureFrameUserData_GameThread();

	// Insert it into the queue
	{
		FScopeLock Lock(&CaptureDataQueueCriticalSection);
		
		// Don't add if we already cached this frame number or newer.
		// Note: Only checks last inserted since frames are expected to increase monotonically and inserted at position 0.
		if (CaptureDataQueue.Num() > 0 && CaptureDataQueue[0].BaseData.SourceFrameNumberRenderThread >= FrameNumber)
		{
			return;
		}

		TRACE_BOOKMARK(TEXT("MediaCapture GT push frame %llu"), FrameNumber);
		CaptureDataQueue.Insert(MoveTemp(CaptureData), 0);
	}
}

void UMediaCapture::OnBeginFrame_GameThread()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MediaCapture::BeginFrame);

	if (ViewExtension && !ViewExtension->IsValid())
	{
		SetState(EMediaCaptureState::Error);
	}

	if (CaptureSource && !CaptureSource->IsValid())
	{
		SetState(EMediaCaptureState::Error);
	}

	if (GetState() == EMediaCaptureState::Error)
	{
		StopCapture(false);
		return;
	}

	if (CaptureSource->GetCaptureType() == UE::MediaCapture::Private::ECaptureType::Immediate && (GetState() == EMediaCaptureState::Capturing))
	{
		AddFrameInfoForCaptureImmediate(GFrameCounter);
	}
}

void UMediaCapture::OnViewportRendered_GameThread(FViewport* Viewport)
{
	if (CaptureSource.IsValid() && CaptureSource->GetSceneViewport().Get() == Viewport)
	{
		ProcessCapture_GameThread();
	}
}

void UMediaCapture::OnEndFrame_GameThread()
{
	using namespace UE::MediaCaptureData;

	TRACE_CPUPROFILER_EVENT_SCOPE(MediaCaptureEndFrame);
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("MediaCapturePipe: %llu"), GFrameCounter % 10));

	if (bPendingOutputResourcesInitialized)
	{
		FlushRenderingCommands();
	}
	
	if (bIsAutoRestartRequired)
	{
		return;
	}

	if (!MediaOutput)
	{
		return;
	}

	if (GetState() == EMediaCaptureState::Error)
	{
		StopCapture(false);
	}

	if (GetState() != EMediaCaptureState::Capturing && GetState() != EMediaCaptureState::StopRequested)
	{
		return;
	}

	if (DesiredCaptureOptions.bSkipFrameWhenRunningExpensiveTasks && !FSlateThrottleManager::Get().IsAllowingExpensiveTasks())
	{
		return;
	}

	if (GetState() == EMediaCaptureState::StopRequested && PendingFrameCount.load() <= 0)
	{
		// All the requested frames have been captured.
		StopCapture(false);
		return;
	}

	// Scene viewport captures are processed during viewport rendering (via OnViewportRendered_GameThread).
	if (!CaptureSource.IsValid() || !CaptureSource->GetSceneViewport().IsValid())
	{
		ProcessCapture_GameThread();
	}
}

bool UMediaCapture::ProcessCapture_RenderThread(const TSharedPtr<UE::MediaCaptureData::FCaptureFrame>& CapturingFrame, UE::MediaCaptureData::FCaptureFrameArgs Args)
{
	using namespace UE::MediaCaptureData;

	TRACE_CPUPROFILER_EVENT_SCOPE(UMediaCapture::ProcessCapture_RenderThread);

	int FrameNumber = -1;
	if (CapturingFrame)
	{
		FrameNumber = CapturingFrame->CaptureBaseData.SourceFrameNumberRenderThread;
	}

	RDG_EVENT_SCOPE_STAT(Args.GraphBuilder, MediaCapture_ProcessCapture, "MediaCapture_ProcessCapture", FrameNumber % 10);
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("Process Capture Render Thread Frame %d"), FrameNumber % 10));

	if (bSimulateError)
	{
		Args.MediaCapture->SetState(EMediaCaptureState::Error);
		bSimulateError = false;
		return false;
	}

	if (CapturingFrame)
	{
		bool bHasCaptureSuceeded = false;
		bool bBeforeCaptureSuccess = true;

		if (Args.RDGResourceToCapture || Args.ResourceToCapture)
		{
			// Register the source texture with the render graph.
			if (!Args.RDGResourceToCapture)
			{
				// If we weren't passed a rdg texture, register the external rhi texture.
				Args.RDGResourceToCapture = Args.GraphBuilder.RegisterExternalTexture(CreateRenderTarget(Args.ResourceToCapture, TEXT("MediaCaptureSourceTexture")));
			}

			if (CapturingFrame->bPrerollFrame)
			{
				// Make sure we're not doing prerolling when capturing a certain number of frames.
				check(!GetDesiredCaptureOptions().bAutostopOnCapture);
			}

			if (CapturingFrame->bPrerollFrame || GetDesiredCaptureOptions().bAutostopOnCapture || !CVarMediaIOPrerollFrames.GetValueOnRenderThread())
			{
				Args.MediaCapture->CaptureRenderPipeline->InitializeResources_RenderThread(CapturingFrame, Args.GraphBuilder, Args.RDGResourceToCapture);
			}

			bool bIsReadyToStartOutput = Args.MediaCapture->IsReadyToOutput();

			if (!CapturingFrame->bPrerollFrame && !bIsReadyToStartOutput && !bLoggedOutputNotReady)
			{
				bLoggedOutputNotReady = true;
				UE_LOGF(LogMediaIOCore, Display, "Skipping frame since capture implementation is not ready to output yet.");
			}

			if (!CapturingFrame->bPrerollFrame && bIsReadyToStartOutput)
			{
				UE_LOGF(LogMediaIOCore, Verbose, "Capturing %ls frame (%d)", FApp::GetTimecode().Frames % 2 == 0 ? TEXT("Even") : TEXT("Odd"), FApp::GetTimecode().Frames);

				if (Args.MediaCapture->ShouldCaptureThisFrame(CapturingFrame->CaptureBaseData))
				{
					// Call the capture frame algo based on the specific type of resource we are using
					if (DesiredOutputResourceType == EMediaCaptureResourceType::Texture)
					{
						if (ensure(CapturingFrame->IsTextureResource()))
						{
							Args.MediaCapture->BeforeFrameCaptured_RenderingThread(CapturingFrame->CaptureBaseData, CapturingFrame->UserData, CapturingFrame->GetTextureResource());
						}
						else
						{
							bBeforeCaptureSuccess = false;
							UE_LOGF(LogMediaIOCore, Error, "The capture will stop for '%ls'. Capture frame was expected to use Texture resource but wasn't.", *Args.MediaCapture->MediaOutputName);
						}
					}
					else
					{
						if (ensure(CapturingFrame->IsBufferResource()))
						{
							Args.MediaCapture->BeforeFrameCaptured_RenderingThread(CapturingFrame->CaptureBaseData, CapturingFrame->UserData, CapturingFrame->GetBufferResource());
						}
						else
						{
							bBeforeCaptureSuccess = false;
							UE_LOGF(LogMediaIOCore, Error, "The capture will stop for '%ls'. Capture frame was expected to use Buffer resource but wasn't.", *Args.MediaCapture->MediaOutputName);
						}
					}

					if (bBeforeCaptureSuccess)
					{
						bHasCaptureSuceeded = UE::MediaCaptureData::FMediaCaptureHelper::CaptureFrame(Args, CapturingFrame);
					}
				}
				else
				{
					// Return the frame to the available pool if we're not gonna capture it.
					bHasCaptureSuceeded = true;
					FrameManager->MarkAvailable(*CapturingFrame);

					UE_LOGF(LogMediaIOCore, Verbose, "=============================================================== Skipping frame %d", FApp::GetTimecode().Frames);
				}
			}
			else
			{
				bHasCaptureSuceeded = true;
				CapturingFrame->bPrerollFrame = false;
				FrameManager->MarkAvailable(*CapturingFrame);
			}
		}
		else
		{
			UE_LOGF(LogMediaIOCore, Error, "The capture will stop for '%ls'. No texture was acquired for the capture.", *Args.MediaCapture->MediaOutputName);
		}
		
		if (bHasCaptureSuceeded == false)
		{
			if (Args.MediaCapture->bIsAutoRestartRequired)
			{
				TWeakObjectPtr<UMediaCapture> Self = this;
				AsyncTask(ENamedThreads::GameThread, [Self]
				{
					TStrongObjectPtr<UMediaCapture> MediaCapture = Self.Pin();
					if (UObjectInitialized() && MediaCapture.IsValid())
					{
						MediaCapture->RestartCapture();
					}
				});
			}
			else
			{
				Args.MediaCapture->SetState(EMediaCaptureState::Error);
			}
		}

		return bHasCaptureSuceeded;
	}

	return false;
}

bool UMediaCapture::ProcessReadyFrame_RenderThread(FRHICommandListImmediate& RHICmdList, UMediaCapture* InMediaCapture, const TSharedPtr<UE::MediaCaptureData::FCaptureFrame>& ReadyFrame)
{
	using namespace UE::MediaCaptureData;
	TRACE_CPUPROFILER_EVENT_SCOPE(UMediaCapture::ProcessReadyFrame_RenderThread);

	bool bWasFrameProcessed = true;
	if (InMediaCapture->GetState() != EMediaCaptureState::Error)
	{
		if (ReadyFrame->bReadbackRequested)
		{
			FRHIGPUMask GPUMask;
#if WITH_MGPU
			GPUMask = RHICmdList.GetGPUMask();

			// If GPUMask is not set to a specific GPU we and since we are reading back the texture, it shouldn't matter which GPU we do this on.
			if (!GPUMask.HasSingleIndex())
			{
				GPUMask = FRHIGPUMask::FromIndex(GPUMask.GetFirstIndex());
			}

			SCOPED_GPU_MASK(RHICmdList, GPUMask);
#endif
			// Lock & read
			void* ColorDataBuffer = nullptr;
			int32 RowStride = 0;

			// If readback is ready, proceed. 
			// If not, proceed with locking only if we are in flush mode since it will block until gpu is idle.
			const bool bIsReadbackReady = InMediaCapture->DesiredCaptureOptions.OverrunAction == EMediaCaptureOverrunAction::Flush || ReadyFrame->IsReadbackReady(GPUMask) == true;
			if (bIsReadbackReady)
			{
				SCOPE_CYCLE_COUNTER(STAT_MediaCapture_RenderThread_LockResource);
				ColorDataBuffer = ReadyFrame->Lock(RHICmdList, RowStride);
			}
			else
			{
				UE_LOGF(LogMediaIOCore, VeryVerbose, "[%ls] - Readback %d not ready. Skipping.", *InMediaCapture->MediaOutputName, ReadyFrame->FrameId);
				bWasFrameProcessed = false;
			}

			if (ColorDataBuffer)
			{
				{
					SCOPE_CYCLE_COUNTER(STAT_MediaCapture_RenderThread_RHI_CaptureCallback)
					TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("MediaCapture Output Frame %lld"), ReadyFrame->CaptureBaseData.SourceFrameNumberRenderThread % 10));

					// The Width/Height of the surface may be different then the DesiredOutputSize : Some underlying implementations enforce a specific stride, therefore
					// there may be padding at the end of each row.
					InMediaCapture->OnFrameCaptured_RenderingThread(ReadyFrame->CaptureBaseData, ReadyFrame->UserData, ColorDataBuffer, InMediaCapture->DesiredOutputSize.X, InMediaCapture->DesiredOutputSize.Y, RowStride);
				}

				ReadyFrame->Unlock();
			}

			if (bIsReadbackReady)
			{
				UE_LOGF(LogMediaIOCore, Verbose, "[%ls - %ls] - Completed pending frame %d.", *InMediaCapture->MediaOutputName, *FThreadManager::GetThreadName(FPlatformTLS::GetCurrentThreadId()), ReadyFrame->FrameId);
				ReadyFrame->bReadbackRequested = false;
				--PendingFrameCount;
				InMediaCapture->FrameManager->CompleteNextPending(*ReadyFrame);
			}
		}
		else if (InMediaCapture->bShouldCaptureRHIResource && ReadyFrame->bDoingGPUCopy)
		{
			if (ReadyFrame->IsTextureResource())
			{
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(UMediaCapture::UnlockDMATexture_RenderThread);
					UnlockDMATexture_RenderThread(ReadyFrame->GetTextureResource());
				}

				TRACE_CPUPROFILER_EVENT_SCOPE(UMediaCapture::RHIResourceCaptured);
				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("MediaCapture Output Frame %lld"), ReadyFrame->CaptureBaseData.SourceFrameNumberRenderThread % 10));
				SCOPE_CYCLE_COUNTER(STAT_MediaCapture_RenderThread_CaptureCallback)
					InMediaCapture->OnRHIResourceCaptured_RenderingThread(RHICmdList, ReadyFrame->CaptureBaseData, ReadyFrame->UserData, ReadyFrame->GetTextureResource());
			}
			else
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(UMediaCapture::RHIResourceCaptured);
				InMediaCapture->OnRHIResourceCaptured_RenderingThread(RHICmdList, ReadyFrame->CaptureBaseData, ReadyFrame->UserData, ReadyFrame->GetBufferResource());
			}

			UE_LOGF(LogMediaIOCore, Verbose, "[%ls - %ls] - Completed pending frame %d.", *InMediaCapture->MediaOutputName, *FThreadManager::GetThreadName(FPlatformTLS::GetCurrentThreadId()), ReadyFrame->FrameId);
			ReadyFrame->bDoingGPUCopy = false;
			InMediaCapture->FrameManager->CompleteNextPending(*ReadyFrame);
			--PendingFrameCount;
		}
	}

	return bWasFrameProcessed;
}

void UMediaCapture::PrintFrameState()
{
	UE_LOGF(LogMediaIOCore, VeryVerbose, "%ls", *FrameManager->GetFramesState());
}

int32 UMediaCapture::GetBytesPerPixel(EPixelFormat InPixelFormat)
{
	//We can capture viewports and render targets. Possible pixel format is limited by that
	switch (InPixelFormat)
	{

	case PF_A8:
	case PF_R8_UINT:
	case PF_R8_SINT:
	case PF_G8:
		{
			return 1;
		}
	case PF_R16_UINT:
	case PF_R16_SINT:
	case PF_R5G6B5_UNORM:
	case PF_R8G8:
	case PF_R16F:
	case PF_R16F_FILTER:
	case PF_V8U8:
	case PF_R8G8_UINT:
	case PF_B5G5R5A1_UNORM:
		{
			return 2;
		}
	case PF_R32_UINT:
	case PF_R32_SINT:
	case PF_R8G8B8A8:
	case PF_A8R8G8B8:
	case PF_FloatR11G11B10:
	case PF_A2B10G10R10:
	case PF_G16R16:
	case PF_G16R16F:
	case PF_G16R16F_FILTER:
	case PF_R32_FLOAT:
	case PF_R16G16_UINT:
	case PF_R8G8B8A8_UINT:
	case PF_R8G8B8A8_SNORM:
	case PF_B8G8R8A8:
	case PF_G16R16_SNORM:
	case PF_FloatRGB: //Equivalent to R11G11B10
	{
			return 4;
	}
	case PF_R16G16B16A16_UINT:
	case PF_R16G16B16A16_SINT:
	case PF_A16B16G16R16:
	case PF_G32R32F:
	case PF_R16G16B16A16_UNORM:
	case PF_R16G16B16A16_SNORM:
	case PF_R32G32_UINT:
	case PF_R64_UINT:
	case PF_FloatRGBA: //Equivalent to R16G16B16A16
	{
			return 8;
	}
	case PF_A32B32G32R32F:
	case PF_R32G32B32A32_UINT:
		{
			return 16;
		}
	default:
		{
			ensureMsgf(false, TEXT("MediaCapture - Pixel format (%d) not handled. Invalid bytes per pixel returned."), InPixelFormat);
			return 0;
		}
	}
}

void UMediaCapture::PostCaptureCleanup()
{
	// Clean up implementation resources.
	PostCaptureCleanupImpl();

	SetCaptureAudioDevice(FAudioDeviceHandle());

	if (CaptureSource)
	{
		CaptureSource->RestoreShowFlags();
		constexpr bool bFlushRenderingCommands = false;
		CaptureSource->ResetSourceBufferSize(bFlushRenderingCommands);
		CaptureSource.Reset();
	}

	DesiredSize = FIntPoint(1280, 720);
	DesiredPixelFormat = EPixelFormat::PF_A2B10G10R10;
	DesiredOutputSize = FIntPoint(1280, 720);
	DesiredOutputPixelFormat = EPixelFormat::PF_A2B10G10R10;
	DesiredCaptureOptions = FMediaCaptureOptions();
	ConversionOperation = EMediaCaptureConversionOperation::NONE;
	SyncPointWatcher.Reset();
	ViewExtension.Reset();
	CaptureRenderPipeline.Reset();
	bLoggedOutputNotReady = false;
	MediaOutputName.Reset();

	bWaitingForStopCapture = false;
}

/* namespace MediaCaptureDetails implementation
*****************************************************************************/
namespace MediaCaptureDetails
{
	bool FindSceneViewportAndLevel(TSharedPtr<FSceneViewport>& OutSceneViewport)
	{
#if WITH_EDITOR
		if (GIsEditor)
		{
			for (const FWorldContext& Context : GEngine->GetWorldContexts())
			{
				if (Context.WorldType == EWorldType::PIE)
				{
					UEditorEngine* EditorEngine = CastChecked<UEditorEngine>(GEngine);
					FSlatePlayInEditorInfo& Info = EditorEngine->SlatePlayInEditorMap.FindChecked(Context.ContextHandle);

					// The PIE window has priority over the regular editor window, so we need to break out of the loop if either of these are found
					if (TSharedPtr<IAssetViewport> DestinationLevelViewport = Info.DestinationSlateViewport.Pin())
					{
						OutSceneViewport = DestinationLevelViewport->GetSharedActiveViewport();
						break;
					}
					else if (Info.SlatePlayInEditorWindowViewport.IsValid())
					{
						OutSceneViewport = Info.SlatePlayInEditorWindowViewport;
						break;
					}
				}
				else if (Context.WorldType == EWorldType::Editor)
				{
					if (FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(LevelEditorName))
					{
						TSharedPtr<IAssetViewport> ActiveLevelViewport = LevelEditorModule->GetFirstActiveViewport();
						if (ActiveLevelViewport.IsValid())
						{
							OutSceneViewport = ActiveLevelViewport->GetSharedActiveViewport();
						}
					}
				}
			}
		}
		else
#endif
		{
			UGameEngine* GameEngine = CastChecked<UGameEngine>(GEngine);
			OutSceneViewport = GameEngine->SceneViewport;
		}

		return (OutSceneViewport.IsValid());
	}

	bool ValidateIsCapturing(const UMediaCapture* CaptureToBeValidated)
	{
		if (CaptureToBeValidated->GetState() != EMediaCaptureState::Capturing && CaptureToBeValidated->GetState() != EMediaCaptureState::Preparing)
		{
			UE_LOG(LogMediaIOCore, Error, TEXT("Can not update the capture. There is no capture currently.\
			Only use UpdateSceneViewport or UpdateTextureRenderTarget2D when the state is Capturing or Preparing"));
			return false;
		}

		return true;
	}

	void ShowSlateNotification()
	{
#if WITH_EDITOR
		if (GIsEditor)
		{
			static double PreviousWarningTime = 0.0;
			const double TimeNow = FPlatformTime::Seconds();
			const double TimeBetweenWarningsInSeconds = 3.0f;

			if (TimeNow - PreviousWarningTime > TimeBetweenWarningsInSeconds)
			{
				FNotificationInfo NotificationInfo(LOCTEXT("MediaCaptureFailedError", "The media failed to capture. Check Output Log for details!"));
				NotificationInfo.ExpireDuration = 2.0f;
				FSlateNotificationManager::Get().AddNotification(NotificationInfo);

				PreviousWarningTime = TimeNow;
			}
		}
#endif // WITH_EDITOR
	}
}

#undef LOCTEXT_NAMESPACE
