// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaCaptureHelper.h"
#include "MediaCaptureRenderPass.h"
#include "MediaCaptureSyncPointWatcher.h"

DECLARE_CYCLE_STAT(TEXT("MediaCapture RenderThread LockResource"), STAT_MediaCaptureHelper_RenderThread_LockResource, STATGROUP_Media);
DECLARE_CYCLE_STAT(TEXT("MediaCapture AnyThread LockResource"), STAT_MediaCaptureHelper_AnyThread_LockResource, STATGROUP_Media);

/** Time spent in media capture sending a frame. */
DECLARE_CYCLE_STAT(TEXT("MediaCapture RenderThread FrameCapture"), STAT_MediaCaptureHelper_RenderThread_FrameCapture, STATGROUP_Media);
DECLARE_CYCLE_STAT(TEXT("MediaCapture RenderThread RHI Capture Callback"), STAT_MediaCaptureHelper_RenderThread_RHI_CaptureCallback, STATGROUP_Media);

DECLARE_GPU_STAT(MediaCapture_CaptureFrame);
DECLARE_GPU_STAT(MediaCapture_SyncPoint);
DECLARE_GPU_STAT(MediaCapture_CustomCapture);
DECLARE_GPU_STAT(MediaCapture_Readback);

static TAutoConsoleVariable<int32> CVarMediaIOCapturePollTaskPriority(
	TEXT("MediaIO.Capture.PollTaskPriority"), static_cast<int32>(LowLevelTasks::ETaskPriority::High),
	TEXT("Priority of the task responsible to poll the render fence"),
	ECVF_RenderThreadSafe);

namespace UE::MediaCaptureData
{
bool FMediaCaptureHelper::AreInputsValid(const UE::MediaCaptureData::FCaptureFrameArgs& Args)
{
	// If it is a simple rgba swizzle we can handle the conversion. Supported formats
	// contained in SupportedRgbaSwizzleFormats. Warning would've been displayed on start of capture.
	if (Args.MediaCapture->GetDesiredPixelFormat() != Args.GetFormat())
	{
		const bool bIsCompressedFormat = IsDXTCBlockCompressedTextureFormat(Args.GetFormat());
		if (!bIsCompressedFormat && (!UMediaCapture::GetSupportedRgbaSwizzleFormats().Contains(Args.GetFormat()) || !Args.MediaCapture->DesiredCaptureOptions.bConvertToDesiredPixelFormat))
		{
			UE_LOGF(LogMediaIOCore, Error, "The capture will stop for '%ls'. The Source pixel format doesn't match with the user requested pixel format. %lsRequested: %ls Source: %ls"
				, *Args.MediaCapture->MediaOutputName
				, (UMediaCapture::GetSupportedRgbaSwizzleFormats().Contains(Args.GetFormat()) && !Args.MediaCapture->DesiredCaptureOptions.bConvertToDesiredPixelFormat) ? TEXT("Please enable \"Convert To Desired Pixel Format\" option in Media Capture settings. ") : TEXT("")
				, GetPixelFormatString(Args.MediaCapture->GetDesiredPixelFormat())
				, GetPixelFormatString(Args.GetFormat()));

			return false;
		}
	}

	if (Args.MediaCapture->DesiredCaptureOptions.ResizeMethod == EMediaCaptureResizeMethod::ResizeInCapturePass &&
		Args.MediaCapture->GetConversionOperation() != EMediaCaptureConversionOperation::CUSTOM)
	{
		UE_LOGF(LogMediaIOCore, Error, "The capture will stop for '%ls'. The resize method of ResizeInCapturePass expects GetConversionOperation() to return EMediaCaptureConversionOperation::CUSTOM"
			, *Args.MediaCapture->MediaOutputName);

		return false;	
	}

	if (Args.MediaCapture->DesiredCaptureOptions.ResizeMethod != EMediaCaptureResizeMethod::ResizeInRenderPass && 
		Args.MediaCapture->DesiredCaptureOptions.ResizeMethod != EMediaCaptureResizeMethod::ResizeInCapturePass)
	{
		bool bFoundSizeMismatch = false;
		FIntPoint RequestSize = FIntPoint::ZeroValue;
		
		if (Args.MediaCapture->DesiredCaptureOptions.Crop == EMediaCaptureCroppingType::None)
		{
			if (Args.SourceViewRect.Area() != 0)
			{
				if (Args.DesiredSize.X != Args.SourceViewRect.Width() || Args.DesiredSize.Y != Args.SourceViewRect.Height())
				{
					RequestSize = { Args.SourceViewRect.Width(), Args.SourceViewRect.Height() };
                    bFoundSizeMismatch = true;
				}
				else
				{
					// If source view rect is passed, it will override the crop passed as argument.
					Args.MediaCapture->DesiredCaptureOptions.Crop = EMediaCaptureCroppingType::Custom;
					Args.MediaCapture->DesiredCaptureOptions.CustomCapturePoint = Args.SourceViewRect.Min;
				}
			}
			else if ((Args.DesiredSize.X != Args.GetSizeX() || Args.DesiredSize.Y != Args.GetSizeY()))
			{
				RequestSize = { Args.DesiredSize };
				bFoundSizeMismatch = true;
			}
		}
		else
		{
			FIntPoint StartCapturePoint = FIntPoint::ZeroValue;
			if (Args.MediaCapture->DesiredCaptureOptions.Crop == EMediaCaptureCroppingType::Custom)
			{
				StartCapturePoint = Args.MediaCapture->DesiredCaptureOptions.CustomCapturePoint;
			}

			if ((Args.DesiredSize.X + StartCapturePoint.X) > Args.GetSizeX() || (Args.DesiredSize.Y + StartCapturePoint.Y) > Args.GetSizeY())
			{
				RequestSize = { Args.DesiredSize };
				bFoundSizeMismatch = true;
			}
		}

		if (bFoundSizeMismatch)
		{
			if (Args.MediaCapture->SupportsAutoRestart() 
				&& Args.MediaCapture->bUseRequestedTargetSize 
				&& Args.MediaCapture->DesiredCaptureOptions.bAutoRestartOnSourceSizeChange)
			{
				Args.MediaCapture->bIsAutoRestartRequired = true;

				UE_LOGF(LogMediaIOCore, Log, "The capture will auto restart for '%ls'. The Source size doesn't match with the user requested size. Requested: %d,%d  Source: %d,%d"
					, *Args.MediaCapture->MediaOutputName
					, RequestSize.X, RequestSize.Y
					, Args.GetSizeX(), Args.GetSizeY());
					
				return false;
			}
			else
			{
				UE_LOGF(LogMediaIOCore, Error, "The capture will stop for '%ls'. The Source size doesn't match with the user requested size. Requested: %d,%d  Source: %d,%d"
					, *Args.MediaCapture->MediaOutputName
					, RequestSize.X, RequestSize.Y
					, Args.GetSizeX(), Args.GetSizeY());

				return false;
			}
		}
	}
				

	return true;
}

bool FMediaCaptureHelper::CaptureFrame(const UE::MediaCaptureData::FCaptureFrameArgs& Args, TSharedPtr<UE::MediaCaptureData::FCaptureFrame> CapturingFrame)
{
	RDG_EVENT_SCOPE_STAT(Args.GraphBuilder, MediaCapture_CaptureFrame, "MediaCapture_CaptureFrame");

	if (!Args.HasValidResource())
	{
		UE_LOGF(LogMediaIOCore, Error, "Can't grab the Texture to capture for '%ls'.", *Args.MediaCapture->MediaOutputName);
		return false;
	}

	// Validate pixel formats and sizes before pursuing
	if (!AreInputsValid(Args))
	{
		return false;
	}

	if (CapturingFrame->IsTextureResource())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UMediaCapture::LockDMATexture_RenderThread);
		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("LockDmaTexture Output Frame %lld"), CapturingFrame->CaptureBaseData.SourceFrameNumberRenderThread % 10));
		Args.MediaCapture->LockDMATexture_RenderThread(CapturingFrame->GetTextureResource());
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UMediaCapture::FrameCapture);
		SCOPE_CYCLE_COUNTER(STAT_MediaCaptureHelper_RenderThread_FrameCapture);

		FRDGTextureRef SourceRGBTexture = Args.RDGResourceToCapture;
		// Final pass output resource used by the current capture method (texture or buffer)
		FRDGViewableResource* FinalPassOutputResource = CapturingFrame->RenderPassResources.GetFinalRDGResource(Args.GraphBuilder);
		
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UMediaCapture::GraphSetup);
			SCOPED_DRAW_EVENTF(Args.GraphBuilder.RHICmdList, MediaCapture, TEXT("MediaCapture"));

			Args.MediaCapture->CaptureRenderPipeline->ExecutePasses_RenderThread(Args, CapturingFrame, Args.GraphBuilder, SourceRGBTexture);
		}

		if (Args.MediaCapture->bShouldCaptureRHIResource == false)
		{
			RDG_EVENT_SCOPE_STAT(Args.GraphBuilder, MediaCapture_Readback, "MediaCapture_Readback");
			TRACE_CPUPROFILER_EVENT_SCOPE(UMediaCapture::EnqueueReadback);

			CapturingFrame->EnqueueCopy(Args.GraphBuilder, FinalPassOutputResource, Args.MediaCapture->UseAnyThreadCapture());
			CapturingFrame->bReadbackRequested = true;
		}
		else
		{
			CapturingFrame->bDoingGPUCopy = true;
		}

		Args.MediaCapture->FrameManager->MarkPending(*CapturingFrame);
		++Args.MediaCapture->PendingFrameCount;

		if (Args.MediaCapture->UseExperimentalScheduling())
		{
			RDG_EVENT_SCOPE_STAT(Args.GraphBuilder, MediaCapture_SyncPoint, "MediaCapture_SyncPoint");

			if (CapturingFrame->IsTextureResource())
			{
				AddSyncPointPass<FTextureCaptureFrame>(Args.GraphBuilder, Args.MediaCapture, CapturingFrame, FinalPassOutputResource);
			}
			else
			{
				AddSyncPointPass<FBufferCaptureFrame>(Args.GraphBuilder, Args.MediaCapture, CapturingFrame, FinalPassOutputResource);
			}
		}
	}

	return true;
}

void FMediaCaptureHelper::ExecuteSyncPointPass(FRHICommandListImmediate& RHICmdList, UMediaCapture* MediaCapture, TSharedPtr<UE::MediaCaptureData::FCaptureFrame> CapturingFrame)
{
	if (CapturingFrame && CapturingFrame->bMediaCaptureActive)
	{
		if (MediaCapture->ShouldCaptureRHIResource())
		{
			MediaCapture->UnlockDMATexture_RenderThread(CapturingFrame->GetTextureResource());

			UE::MediaCaptureData::FPendingCaptureData NewCaptureData;
			NewCaptureData.CapturingFrame = CapturingFrame;
			MediaCapture->SyncPointWatcher->QueuePendingCapture(MoveTemp(NewCaptureData));
		}
		else
		{
			// Get available sync handler to create a sync point
			TSharedPtr<UMediaCapture::FMediaCaptureSyncData> SyncDataPtr = MediaCapture->GetAvailableSyncHandler();
			if (ensure(SyncDataPtr))
			{
				// This will happen after the conversion pass has completed
				RHICmdList.WriteGPUFence(SyncDataPtr->RHIFence);
				SyncDataPtr->bIsBusy = true;

				// Queue capture data and our thread will wait (poll) and continue the process of providing a new texture
				{
					UE::MediaCaptureData::FPendingCaptureData NewCaptureData;
					NewCaptureData.SyncHandler = SyncDataPtr;
					NewCaptureData.CapturingFrame = CapturingFrame;
					MediaCapture->SyncPointWatcher->QueuePendingCapture(MoveTemp(NewCaptureData));
				}
			}
			else
			{
				UE_LOGF(LogMediaIOCore, Error, "GetAvailableSyncHandler of MediaCapture '%ls' failed to provide a fence, the captured buffers may not become available anymore.",
					*MediaCapture->MediaOutputName);
			}
		}
	}
}

void FMediaCaptureHelper::OnReadbackComplete(UMediaCapture* MediaCapture, TSharedPtr<UE::MediaCaptureData::FCaptureFrame> ReadyFrame)
{
	UE_LOGF(LogMediaIOCore, Verbose, "[%ls - %ls] - Processing pending frame %d", *MediaCapture->GetMediaOutputName(), *FThreadManager::GetThreadName(FPlatformTLS::GetCurrentThreadId()), ReadyFrame->GetId());
	TRACE_CPUPROFILER_EVENT_SCOPE(UMediaCapture::OnReadbackComplete);

	{
		ON_SCOPE_EXIT
		{
			if (IsInRenderingThread())
			{
				--MediaCapture->WaitingForRenderCommandExecutionCounter;
			}
		};

		// Path where resource ready callback (readback / rhi capture) is on render thread (old method)
		if (IsInRenderingThread())
		{
			FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

			// Scoped gpu mask shouldn't be needed for readback since we specify the gpu mask used during copy when we lock
			// Keeping it for old render thread path
			FRHIGPUMask GPUMask;
#if WITH_MGPU
			GPUMask = RHICmdList.GetGPUMask();

			// If GPUMask is not set to a specific GPU we and since we are reading back the texture, it shouldn't matter which GPU we do this on.
			if (!GPUMask.HasSingleIndex())
			{
				GPUMask = FRHIGPUMask::FromIndex(GPUMask.GetFirstIndex());
			}
			SCOPED_GPU_MASK(RHICmdList, GPUMask);
#endif

			// Are we doing a GPU Direct transfer
			if (MediaCapture->ShouldCaptureRHIResource())
			{
				if (ReadyFrame->IsTextureResource())
				{
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(UMediaCapture::UnlockDMATexture_RenderThread);
						MediaCapture->UnlockDMATexture_RenderThread(ReadyFrame->GetTextureResource());
					}

					TRACE_CPUPROFILER_EVENT_SCOPE(UMediaCapture::RHIResourceCaptured);
					TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("MediaCapture Output Frame %lld"), ReadyFrame->CaptureBaseData.SourceFrameNumberRenderThread % 10));
					SCOPE_CYCLE_COUNTER(STAT_MediaCaptureHelper_RenderThread_RHI_CaptureCallback);

					MediaCapture->OnRHIResourceCaptured_RenderingThread(RHICmdList, ReadyFrame->CaptureBaseData, ReadyFrame->UserData, ReadyFrame->GetTextureResource());
				}
				else
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(UMediaCapture::RHIResourceCaptured);
					MediaCapture->OnRHIResourceCaptured_RenderingThread(RHICmdList, ReadyFrame->CaptureBaseData, ReadyFrame->UserData, ReadyFrame->GetBufferResource());
				}
			}
			else
			{
				// Lock & read
				void* ColorDataBuffer = nullptr;
				int32 RowStride = 0;

				// Readback should be ready since we're after the sync point.
				SCOPE_CYCLE_COUNTER(STAT_MediaCaptureHelper_RenderThread_LockResource);
				ColorDataBuffer = ReadyFrame->Lock(RHICmdList, RowStride);

				if (ensure(ColorDataBuffer))
				{
					{
						TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("MediaCapture Output Frame %d"), ReadyFrame->GetId() % 10));
						SCOPE_CYCLE_COUNTER(STAT_MediaCaptureHelper_AnyThread_LockResource)
							MediaCapture->OnFrameCaptured_RenderingThread(ReadyFrame->CaptureBaseData, ReadyFrame->UserData, ColorDataBuffer, MediaCapture->DesiredOutputSize.X, MediaCapture->DesiredOutputSize.Y, RowStride);
					}

					ReadyFrame->Unlock();
				}
			}
		}
		else
		{
			// Are we doing a GPU Direct transfer
			if (MediaCapture->ShouldCaptureRHIResource())
			{
				if (ReadyFrame->IsTextureResource())
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(UMediaCapture::RHIResourceCaptured);
					TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("MediaCapture Output Frame %lld"), ReadyFrame->CaptureBaseData.SourceFrameNumberRenderThread % 10));
					SCOPE_CYCLE_COUNTER(STAT_MediaCaptureHelper_RenderThread_RHI_CaptureCallback)

					MediaCapture->OnRHIResourceCaptured_AnyThread(ReadyFrame->CaptureBaseData, ReadyFrame->UserData, ReadyFrame->GetTextureResource());
				}
				else
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(UMediaCapture::RHIResourceCaptured);
					MediaCapture->OnRHIResourceCaptured_AnyThread(ReadyFrame->CaptureBaseData, ReadyFrame->UserData, ReadyFrame->GetBufferResource());
				}
			}
			else
			{
				// Lock & read
				void* ColorDataBuffer = nullptr;
				int32 RowStride = 0;

				// Readback should be ready since we're after the sync point.
				SCOPE_CYCLE_COUNTER(STAT_MediaCaptureHelper_RenderThread_LockResource);
				ColorDataBuffer = ReadyFrame->Lock_Unsafe(RowStride);

				if (ensure(ColorDataBuffer))
				{
					{
						TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("MediaCapture Output Frame %d"), ReadyFrame->GetId() % 10));
						SCOPE_CYCLE_COUNTER(STAT_MediaCaptureHelper_AnyThread_LockResource)

						UMediaCapture::FMediaCaptureResourceData ResourceData;
						ResourceData.Buffer = ColorDataBuffer;
						ResourceData.Width = MediaCapture->DesiredOutputSize.X;
						ResourceData.Height = MediaCapture->DesiredOutputSize.Y;
						ResourceData.BytesPerRow = RowStride;
						MediaCapture->OnFrameCaptured_AnyThread(ReadyFrame->CaptureBaseData, ReadyFrame->UserData, MoveTemp(ResourceData));
					}

					ReadyFrame->Unlock_Unsafe();
				}
			}
		}

		ReadyFrame->bDoingGPUCopy = false;
		ReadyFrame->bReadbackRequested = false;

		UE_LOGF(LogMediaIOCore, Verbose, "[%ls - %ls] - Completed pending frame %d.", *MediaCapture->GetMediaOutputName(), *FThreadManager::GetThreadName(FPlatformTLS::GetCurrentThreadId()), ReadyFrame->GetId());
		MediaCapture->FrameManager->CompleteNextPending(*ReadyFrame);
		--MediaCapture->PendingFrameCount;
	};
}

}
