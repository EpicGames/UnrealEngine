// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaIOFrameManager.h"

#include "HAL/ThreadManager.h"
#include "MediaCapture.h"
#include "MediaIOCoreModule.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "RenderGraphBuilder.h"
#include "Stats/Stats.h"
#include "UObject/ObjectPtr.h"

namespace UE::MediaCaptureData
{
	/** Helper class to be able to friend it and call methods on input media capture */
	class FMediaCaptureHelper
	{
	public:
		friend class FSyncPointWatcher;
		
		/** Validates the input and output size and format compatibility. */
		static bool AreInputsValid(const UE::MediaCaptureData::FCaptureFrameArgs& Args);

		/** Kickstart the capture process for a frame. */
		static bool CaptureFrame(const UE::MediaCaptureData::FCaptureFrameArgs& Args, TSharedPtr<UE::MediaCaptureData::FCaptureFrame> CapturingFrame);

		/** Add a sync point pass that will wait for a readback to complete before dispatching the result to the right capture callback on the media capture. */
		template <typename CaptureType>
		static void AddSyncPointPass(FRDGBuilder& GraphBuilder, UMediaCapture* MediaCapture, TSharedPtr<UE::MediaCaptureData::FCaptureFrame> CapturingFrame, FRDGViewableResource* OutputResource)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MediaCaptureSyncPoint);
			TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("MediaCaptureSyncPoint_%lld"), CapturingFrame->CaptureBaseData.SourceFrameNumberRenderThread % 10));

			// Initialize sync handlers only the first time. 
			if (MediaCapture->bSyncHandlersInitialized == false)
			{
				MediaCapture->InitializeSyncHandlers_RenderThread();
			}

			// Add buffer output as a parameter to depend on the compute shader pass
			typename CaptureType::PassParametersType* PassParameters = GraphBuilder.AllocParameters<typename CaptureType::PassParametersType>();
			PassParameters->Resource = static_cast<typename CaptureType::FOutputResourceType>(OutputResource);

			GraphBuilder.AddPass(
				RDG_EVENT_NAME("MediaCaptureCopySyncPass"),
				PassParameters,
				ERDGPassFlags::Copy | ERDGPassFlags::NeverCull,
				[MediaCapture, CapturingFrame](FRHICommandListImmediate& RHICmdList)
				{
					FMediaCaptureHelper::ExecuteSyncPointPass(RHICmdList, MediaCapture, CapturingFrame);
				});
		}

	private:
		/** Implementation of the sync point pass that will poll a gpu fence until it's cleared. */
		static void ExecuteSyncPointPass(FRHICommandListImmediate& RHICmdList, UMediaCapture* MediaCapture, TSharedPtr<UE::MediaCaptureData::FCaptureFrame> CapturingFrame);
		/**
		 * Function called when the last copy of the media capture pipeline is completed. 
		 * This will handle calling the right capture callback on the MediaCapture (ie. OnRHIResourceCaptured_RenderingThread)
		 */
		static void OnReadbackComplete(UMediaCapture* MediaCapture, TSharedPtr<UE::MediaCaptureData::FCaptureFrame> ReadyFrame);
	};
}
