// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaCaptureRenderPass.h"

#include "CoreTypes.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Engine/Engine.h"
#include "MediaCapture.h"
#include "MediaCaptureHelper.h"
#include "OpenColorIOConfiguration.h"
#include "OpenColorIOShader.h"
#include "OpenColorIOShared.h"
#include "PixelShaderUtils.h"
#include "RenderGraphEvent.h"
#include "RenderGraphResources.h"
#include "SceneView.h"
#include "ScreenPass.h"
#include "ShaderParameterMacros.h"

DECLARE_GPU_STAT(MediaCapture_Resample);
DECLARE_GPU_STAT(MediaCapture_ColorConversion);
DECLARE_GPU_STAT(MediaCapture_Conversion);
DECLARE_GPU_STAT(MediaCapture_Uncompression);

namespace UE::MediaCapture::Resample
{
	/**
	 * Pixel shader to resample a texture using 4 samples per pixel.
	 */
	class FMediaCaptureResamplePS : public FGlobalShader
	{
	public:
		DECLARE_GLOBAL_SHADER(FMediaCaptureResamplePS);
		SHADER_USE_PARAMETER_STRUCT(FMediaCaptureResamplePS, FGlobalShader);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Input)
			SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Output)
			SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture) 
			SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
			RENDER_TARGET_BINDING_SLOTS()
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
		{
			return true;
		}

		/** Allocates and setup shader parameter in the incoming graph builder */
		static FMediaCaptureResamplePS::FParameters* AllocateAndSetParameters(FRDGBuilder& GraphBuilder, FRDGTextureRef RGBATexture, FRDGTextureRef OutputTexture)
		{
			FMediaCaptureResamplePS::FParameters* Parameters = GraphBuilder.AllocParameters<FMediaCaptureResamplePS::FParameters>();

			Parameters->Input = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(RGBATexture));
			Parameters->Output = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(OutputTexture));
			Parameters->InputTexture = RGBATexture;
			Parameters->InputSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			Parameters->RenderTargets[0] = FRenderTargetBinding{ OutputTexture, ERenderTargetLoadAction::ENoAction };

			return Parameters;
		}
	};
	
	IMPLEMENT_GLOBAL_SHADER(FMediaCaptureResamplePS, "/MediaIOShaders/MediaIO.usf", "MediaDownsample", SF_Pixel);
	
	/** Adds a Resample pass to the graph builder. */
	void AddResamplePass(FRDGBuilder& GraphBuilder, const FScreenPassTexture& InputTexture, FRDGTextureRef OutputTexture)
	{
		check(InputTexture.Texture);
		FScreenPassRenderTarget Input;
		const FIntVector InputSize = InputTexture.Texture->Desc.GetSize();
		Input.Texture = InputTexture.Texture;
		Input.ViewRect = FIntRect{ 0, 0, InputSize.X, InputSize.Y };
		Input.LoadAction = ERenderTargetLoadAction::ENoAction;
		Input.UpdateVisualizeTextureExtent();

		FScreenPassRenderTarget Output;
		const FIntVector OutputSize = OutputTexture->Desc.GetSize();
		Output.Texture = OutputTexture;
		Output.ViewRect = FIntRect{ 0, 0, OutputSize.X, OutputSize.Y };
		Output.LoadAction = ERenderTargetLoadAction::ENoAction;
		Output.UpdateVisualizeTextureExtent();

		FMediaCaptureResamplePS::FParameters* PassParameters = FMediaCaptureResamplePS::AllocateAndSetParameters(GraphBuilder, InputTexture.Texture, OutputTexture);
		
		const FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		const TShaderMapRef<FMediaCaptureResamplePS> PixelShader(GlobalShaderMap);
		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			GlobalShaderMap,
			RDG_EVENT_NAME("MediaCapture Resample(%s) %dx%d -> %dx%d",
				Output.Texture->Name,
				Input.ViewRect.Width(), Input.ViewRect.Height(),
				Output.ViewRect.Width(), Output.ViewRect.Height()),
			PixelShader,
			PassParameters,
			Output.ViewRect);
	}

}

namespace UE::MediaCapture::ColorConversion
{
	/** Create a render pass resource struct from the media capture's color conversion settings. */
	FOpenColorIORenderPassResources GetColorConversionResources(const UMediaCapture* MediaCapture)
	{
		FOpenColorIORenderPassResources OCIOResources;

		if (MediaCapture && MediaCapture->GetDesiredCaptureOptions().ColorConversionSettings.IsValid())
		{
			OCIOResources = FOpenColorIORendering::GetRenderPassResources(MediaCapture->GetDesiredCaptureOptions().ColorConversionSettings, GMaxRHIFeatureLevel);
		}

		return OCIOResources;
	}
}

namespace UE::MediaCapture
{
	/** Gather information needed for a copy based on the crop and source position capture settings. */
	static void GetCopyInfo(const UE::MediaCaptureData::FCaptureFrameArgs& Args, FRHICopyTextureInfo& OutCopyInfo, FVector2D& OutSizeU, FVector2D& OutSizeV)
	{
		// Default to no crop
		OutSizeU = { 0.0f, 1.0f };
		OutSizeV = { 0.0f, 1.0f };
		OutCopyInfo.Size = FIntVector(Args.DesiredSize.X, Args.DesiredSize.Y, 1);
		if (Args.MediaCapture->GetDesiredCaptureOptions().Crop != EMediaCaptureCroppingType::None)
		{
			switch (Args.MediaCapture->GetDesiredCaptureOptions().Crop)
			{
			case EMediaCaptureCroppingType::Center:
				OutCopyInfo.SourcePosition = FIntVector((Args.GetSizeX() - Args.DesiredSize.X) / 2, (Args.GetSizeY() - Args.DesiredSize.Y) / 2, 0);
				break;
			case EMediaCaptureCroppingType::TopLeft:
				break;
			case EMediaCaptureCroppingType::Custom:
				OutCopyInfo.SourcePosition = FIntVector(Args.MediaCapture->GetDesiredCaptureOptions().CustomCapturePoint.X, Args.MediaCapture->GetDesiredCaptureOptions().CustomCapturePoint.Y, 0);
				break;
			default:
				break;
			}

			OutSizeU.X = (float)(OutCopyInfo.SourcePosition.X) / (float)Args.GetSizeX();
			OutSizeU.Y = (float)(OutCopyInfo.SourcePosition.X + OutCopyInfo.Size.X) / (float)Args.GetSizeX();
			OutSizeV.X = (float)(OutCopyInfo.SourcePosition.Y) / (float)Args.GetSizeY();
			OutSizeV.Y = (float)(OutCopyInfo.SourcePosition.Y + OutCopyInfo.Size.Y) / (float)Args.GetSizeY();
		}
	}

	/** Creates the necessary render target for the color conversion pass. */
	FRenderPass::TRDGResource InitializeColorConversionPassOutputTexture(const FInitializePassOutputArgs& Args, uint32 FrameId)
	{
		// The conversion operation only supports textures at the moment.
		check(Args.InputResource && Args.InputResource->Type == ERDGViewableResourceType::Texture);

		// Copy the resource's description from the last pass's output..
		FRDGTextureDesc ColorConversionOutputTextureDesc = static_cast<FRDGTextureRef>(Args.InputResource)->Desc;
		ColorConversionOutputTextureDesc.ClearValue = FClearValueBinding(FLinearColor::White);
		ColorConversionOutputTextureDesc.Reset();

		TRefCountPtr<IPooledRenderTarget> RenderTarget = AllocatePooledTexture(ColorConversionOutputTextureDesc, TEXT("MediaCapture ColorConversion RenderTarget"));
		return RenderTargetResource(MoveTemp(RenderTarget));
	}

	/** Creates the necessary render target for the Resample pass. */
	FRenderPass::TRDGResource InitializeResamplePassOutputTexture(const FInitializePassOutputArgs& Args, uint32 FrameId)
	{
		// The conversion operation only supports textures at the moment.
		check(Args.InputResource && Args.InputResource->Type == ERDGViewableResourceType::Texture);

		FRDGTextureDesc ResampleOutputTextureDesc = static_cast<FRDGTextureRef>(Args.InputResource)->Desc;
		ResampleOutputTextureDesc.Reset();
		ResampleOutputTextureDesc.Extent.X = Args.MediaCapture->GetDesiredSize().X;
		ResampleOutputTextureDesc.Extent.Y = Args.MediaCapture->GetDesiredSize().Y;
		ResampleOutputTextureDesc.Flags = TexCreate_None;
		ResampleOutputTextureDesc.Flags |= TexCreate_RenderTargetable | TexCreate_UAV | TexCreate_NoFastClear;
		ResampleOutputTextureDesc.ClearValue = FClearValueBinding(FLinearColor(0, 0, 0, 0));

		TRefCountPtr<IPooledRenderTarget> RenderTarget = AllocatePooledTexture(ResampleOutputTextureDesc, TEXT("MediaCapture Resample RenderTarget"));
		return RenderTargetResource(MoveTemp(RenderTarget));
	}

	void FRenderPass::AddColorConversionPass(const UE::MediaCaptureData::FCaptureFrameArgs& Args, const TSharedPtr<UE::MediaCaptureData::FCaptureFrame> CapturingFrame, const FColorConversionPassArgs& ConversionPassArgs, FRDGViewableResource* OutputResource)
	{
		ensure(Args.MediaCapture->GetDesiredCaptureOptions().ColorConversionSettings.IsValid());

		// FOpenColorIORendering::AddPass_RenderThread only supports outputting to a texture at the moment.
		check(OutputResource->Type == FRDGTexture::StaticType);

		// Rectangle area to use from source
		const FIntRect ViewRect(ConversionPassArgs.CopyInfo.GetSourceRect());

		//Dummy ViewFamily/ViewInfo created to use built in Draw Screen/Texture Pass
		FSceneViewFamily ViewFamily(FSceneViewFamily::ConstructionValues(nullptr, nullptr, FEngineShowFlags(ESFIM_Game))
			.SetTime(FGameTime()));

		FSceneViewInitOptions ViewInitOptions;
		ViewInitOptions.ViewFamily = &ViewFamily;
		ViewInitOptions.SetViewRectangle(ViewRect);
		ViewInitOptions.ViewOrigin = FVector::ZeroVector;
		ViewInitOptions.ViewRotationMatrix = FMatrix::Identity;
		ViewInitOptions.ProjectionMatrix = FMatrix::Identity;

		constexpr float DefaultDisplayGamma = 2.2f;

		check(GEngine->DisplayGamma != 0.f)
		const float InverseGamma = DefaultDisplayGamma / GEngine->DisplayGamma;
		float AppliedGamma = DefaultDisplayGamma;

		/**
		 *  Depending on where we land in the post processing pipeline, we might need to apply an inverse gamma correction.
		 */
		switch (Args.MediaCapture->GetDesiredCaptureOptions().CapturePhase)
		{
		case EMediaCapturePhase::BeforePostProcessing:
			AppliedGamma = DefaultDisplayGamma;
			break;
		case EMediaCapturePhase::AfterMotionBlur:
			AppliedGamma = DefaultDisplayGamma;
			break;
		case EMediaCapturePhase::AfterToneMap: // This is where the default gamma is applied, so for any step after it, we need to reverse it before applying OCIO.
		case EMediaCapturePhase::AfterFXAA:
		case EMediaCapturePhase::EndFrame:
			AppliedGamma = InverseGamma;
			break;
		default:
			checkNoEntry();
			break;
		}

		RDG_EVENT_SCOPE_STAT(Args.GraphBuilder, MediaCapture_ColorConversion, "MediaCapture_ColorConversion");

		FOpenColorIORendering::AddPass_RenderThread(
			Args.GraphBuilder,
			FScreenPassViewInfo(),
			GMaxRHIFeatureLevel,
			FScreenPassTexture(ConversionPassArgs.SourceRGBTexture),
			FScreenPassRenderTarget((FRDGTextureRef)OutputResource, ERenderTargetLoadAction::EClear),
			ConversionPassArgs.OCIOResources,
			AppliedGamma
		);
	}
	
	FRenderPass FRenderPass::CreateResamplePass()
	{
		FRenderPass ResamplePass;
		ResamplePass.Name = "Resample";
		ResamplePass.OutputType = ERDGViewableResourceType::Texture;
		ResamplePass.InitializePassOutputDelegate = FRenderPass::FInitializePassOutput::CreateStatic(&InitializeResamplePassOutputTexture);
		ResamplePass.ShouldExecutePassDelegate = FRenderPass::FShouldExecutePass::CreateLambda([](const UE::MediaCaptureData::FCaptureFrameArgs& Args, FRDGViewableResource* InputResource)
		{
			check(InputResource->Type == ERDGViewableResourceType::Texture);
			FRDGTextureRef SourceRGBTexture = static_cast<FRDGTextureRef>(InputResource);
			return SourceRGBTexture->Desc.Extent != Args.DesiredSize;
		});

		ResamplePass.ExecutePassDelegate = FRenderPass::FExecutePass::CreateLambda([](const UE::MediaCaptureData::FCaptureFrameArgs& Args, const TSharedPtr<UE::MediaCaptureData::FCaptureFrame>& CapturingFrame, FRDGViewableResource* InputResource, FRDGViewableResource* OutputTexture)
		{
			check(InputResource->Type == ERDGViewableResourceType::Texture);

			FRHICopyTextureInfo CopyInfo;
			FVector2D SizeU;
			FVector2D SizeV;
			GetCopyInfo(Args, CopyInfo, SizeU, SizeV);

			const FBaseRenderPassArgs ColorArgs { static_cast<FRDGTextureRef>(InputResource), CopyInfo };
			AddTextureResamplePass(Args, ColorArgs, OutputTexture);
		});

		return ResamplePass;
	}


	FRenderPass FRenderPass::CreateColorConversionPass(const UMediaCapture* MediaCapture)
	{
		FRenderPass ColorConversionPass;
		ColorConversionPass.Name = "ColorConversion";
		ColorConversionPass.OutputType = ERDGViewableResourceType::Texture;
		ColorConversionPass.InitializePassOutputDelegate = FRenderPass::FInitializePassOutput::CreateStatic(&InitializeColorConversionPassOutputTexture);
		ColorConversionPass.ExecutePassDelegate = FRenderPass::FExecutePass::CreateLambda([CachedOCIOResources = UE::MediaCapture::ColorConversion::GetColorConversionResources(MediaCapture)](const UE::MediaCaptureData::FCaptureFrameArgs& Args, const TSharedPtr<UE::MediaCaptureData::FCaptureFrame>& CapturingFrame, FRDGViewableResource* InputResource, FRDGViewableResource* OutputTexture)
		{
			check(InputResource->Type == ERDGViewableResourceType::Texture);

			FRHICopyTextureInfo CopyInfo;
			FVector2D SizeU;
			FVector2D SizeV;
			GetCopyInfo(Args, CopyInfo, SizeU, SizeV);
			
			FColorConversionPassArgs ColorArgs;
			ColorArgs.SourceRGBTexture = static_cast<FRDGTextureRef>(InputResource);
			ColorArgs.CopyInfo = CopyInfo;
			ColorArgs.OCIOResources = FOpenColorIORenderPassResources{CachedOCIOResources}; 
			AddColorConversionPass(Args, CapturingFrame, ColorArgs, OutputTexture);
		});

		return ColorConversionPass;
	}

	FRenderPass FRenderPass::CreateFormatConversionPass(const UMediaCapture* MediaCapture)
	{
		FRenderPass ConversionPass;
		ConversionPass.OutputType = MediaCapture->DesiredOutputResourceType == EMediaCaptureResourceType::Texture ? ERDGViewableResourceType::Texture : ERDGViewableResourceType::Buffer;
		if (ConversionPass.OutputType == ERDGViewableResourceType::Texture)
		{
			ConversionPass.InitializePassOutputDelegate = FRenderPass::FInitializePassOutput::CreateLambda([](const UE::MediaCapture::FInitializePassOutputArgs& Args, uint32 FrameId)
			{
				TRefCountPtr<IPooledRenderTarget> RenderTarget = AllocatePooledTexture(Args.MediaCapture->DesiredOutputTextureDescription, TEXT("MediaCapture Output RenderTarget"));
				return RenderTargetResource(MoveTemp(RenderTarget));
			});
		}
		else
		{
			ConversionPass.InitializePassOutputDelegate = FRenderPass::FInitializePassOutput::CreateLambda([](const UE::MediaCapture::FInitializePassOutputArgs& Args, uint32 FrameId)
			{
				TRefCountPtr<FRDGPooledBuffer> Buffer = AllocatePooledBuffer(Args.MediaCapture->DesiredOutputBufferDescription, TEXT("MediaCapture Output Buffer"));
				return BufferResource(MoveTemp(Buffer));
			});
		}
		
		if (MediaCapture->GetConversionOperation() == EMediaCaptureConversionOperation::CUSTOM)
		{
			ConversionPass.Name = "CustomConversion";
			ConversionPass.ExecutePassDelegate = FRenderPass::FExecutePass::CreateLambda([](const UE::MediaCaptureData::FCaptureFrameArgs& Args, const TSharedPtr<UE::MediaCaptureData::FCaptureFrame>& CapturingFrame, FRDGViewableResource* InputResource, FRDGViewableResource* OutputResource)
			{
				// We only support Texture inputs
				check(InputResource->Type == ERDGViewableResourceType::Texture);

				FRHICopyTextureInfo CopyInfo;
				FVector2D SizeU;
				FVector2D SizeV;
				GetCopyInfo(Args, CopyInfo, SizeU, SizeV);
				
				if (CapturingFrame->IsTextureResource())
				{
					Args.MediaCapture->OnCustomCapture_RenderingThread(Args.GraphBuilder, CapturingFrame->CaptureBaseData, CapturingFrame->UserData
						, static_cast<FRDGTextureRef>(InputResource), static_cast<FRDGTextureRef>(OutputResource), CopyInfo, SizeU, SizeV);
				}
				else
				{
					Args.MediaCapture->OnCustomCapture_RenderingThread(Args.GraphBuilder, CapturingFrame->CaptureBaseData, CapturingFrame->UserData
						, static_cast<FRDGTextureRef>(InputResource), static_cast<FRDGBufferRef>(OutputResource), CopyInfo, SizeU, SizeV);
				}
			});
			
		}
		else
		{
			ConversionPass.Name = "Conversion";
			ConversionPass.ExecutePassDelegate = FRenderPass::FExecutePass::CreateLambda([](const UE::MediaCaptureData::FCaptureFrameArgs& Args, const TSharedPtr<UE::MediaCaptureData::FCaptureFrame>& CapturingFrame, FRDGViewableResource* InputResource, FRDGViewableResource* OutputResource)
			{
				// At the moment we only support Texture inputs
				check(InputResource->Type == ERDGViewableResourceType::Texture);
				FRDGTextureRef InputTexture = static_cast<FRDGTextureRef>(InputResource);

				FRHICopyTextureInfo CopyInfo;
				FVector2D SizeU;
				FVector2D SizeV;
				GetCopyInfo(Args, CopyInfo, SizeU, SizeV);

				// If true, we will need to go through our different shader to convert from source format to out format (i.e RGB to YUV)
				const bool bRequiresFormatConversion = Args.MediaCapture->DesiredPixelFormat != Args.GetFormat() || Args.MediaCapture->DesiredPixelFormat != InputTexture->Desc.Format;

				RDG_EVENT_SCOPE_STAT(Args.GraphBuilder, MediaCapture_Conversion, "MediaCapture_Conversion");
				TRACE_CPUPROFILER_EVENT_SCOPE(UMediaCapture::FormatConversion);
				
				FConversionPassArgs ConversionPassArgs;
				ConversionPassArgs.SourceRGBTexture = InputTexture;
				ConversionPassArgs.CopyInfo = CopyInfo;
				ConversionPassArgs.bRequiresFormatConversion = bRequiresFormatConversion;
				ConversionPassArgs.SizeU = SizeU;
				ConversionPassArgs.SizeV = SizeV;
				
				AddConversionPass(Args, ConversionPassArgs, OutputResource);
			});
		}

		return ConversionPass;
	}

	FRenderPass FRenderPass::CreateUncompressionPass(const UMediaCapture* MediaCapture)
	{
		FRenderPass UncompressionPass;
		UncompressionPass.Name = "Uncompression";
		UncompressionPass.OutputType = ERDGViewableResourceType::Texture;
		UncompressionPass.ShouldExecutePassDelegate = FRenderPass::FShouldExecutePass::CreateLambda([](const UE::MediaCaptureData::FCaptureFrameArgs& Args, FRDGViewableResource* InputResource)
		{
			check(InputResource->Type == ERDGViewableResourceType::Texture);
			FRDGTextureRef SourceRGBTexture = static_cast<FRDGTextureRef>(InputResource);
			return IsBlockCompressedFormat(SourceRGBTexture->Desc.Format);
		});
		UncompressionPass.InitializePassOutputDelegate = FRenderPass::FInitializePassOutput::CreateLambda([](const UE::MediaCapture::FInitializePassOutputArgs& Args, uint32 FrameId)
		{
			check(Args.InputResource && Args.InputResource->Type == ERDGViewableResourceType::Texture);
			FRDGTextureDesc InputResourceDesc = static_cast<FRDGTextureRef>(Args.InputResource)->Desc;

			// Capture render pipeline still creates the pass resources even if it doesn't execute, and the format of the render target created here
			// is used to determine the format of passes down the line, so if the format does not need uncompression, simply pass it through.
			EPixelFormat OutputFormat = IsBlockCompressedFormat(InputResourceDesc.Format) ? PF_FloatRGBA : InputResourceDesc.Format;

			FRDGTextureDesc UncompressedOutputTextureDesc = InputResourceDesc;
			UncompressedOutputTextureDesc.Reset();
			UncompressedOutputTextureDesc.NumMips = 1;
			UncompressedOutputTextureDesc.Flags = TexCreate_None;
			UncompressedOutputTextureDesc.Flags |= TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_UAV | TexCreate_NoFastClear;
			UncompressedOutputTextureDesc.ClearValue = FClearValueBinding(FLinearColor(0, 0, 0, 0));
			UncompressedOutputTextureDesc.Format = OutputFormat;

			TRefCountPtr<IPooledRenderTarget> RenderTarget = AllocatePooledTexture(UncompressedOutputTextureDesc, TEXT("MediaCapture Uncompressed Output"));
			return RenderTargetResource(MoveTemp(RenderTarget));
		});

		UncompressionPass.ExecutePassDelegate = FRenderPass::FExecutePass::CreateLambda([](const UE::MediaCaptureData::FCaptureFrameArgs& Args, const TSharedPtr<UE::MediaCaptureData::FCaptureFrame>& CapturingFrame, FRDGViewableResource* InputResource, FRDGViewableResource* OutputResource)
		{
			// At the moment we only support Texture inputs
			check(InputResource->Type == ERDGViewableResourceType::Texture);

			FRHICopyTextureInfo CopyInfo;
			FVector2D SizeU;
			FVector2D SizeV;
			GetCopyInfo(Args, CopyInfo, SizeU, SizeV);

			RDG_EVENT_SCOPE_STAT(Args.GraphBuilder, MediaCapture_Uncompression, "MediaCapture_Uncompression");
			TRACE_CPUPROFILER_EVENT_SCOPE(UMediaCapture::Uncompression);
			
			FBaseRenderPassArgs UncompressionPassArgs;
			UncompressionPassArgs.SourceRGBTexture = static_cast<FRDGTextureRef>(InputResource);
			UncompressionPassArgs.CopyInfo = CopyInfo;
			
			AddUncompressionPass(Args, UncompressionPassArgs, OutputResource);
		});
		
		return UncompressionPass;
	}

	/** Create the necessary render passes needed for the current media capture configuration. */
	TArray<FRenderPass> FRenderPass::CreateRenderPasses(const UMediaCapture* MediaCapture)
	{
		TArray<FRenderPass> RenderPasses;

		RenderPasses.Add(CreateUncompressionPass(MediaCapture));

		if (MediaCapture->GetDesiredCaptureOptions().ResizeMethod == EMediaCaptureResizeMethod::ResizeInRenderPass)
		{
			if (MediaCapture->GetDesiredCaptureOptions().Crop != EMediaCaptureCroppingType::None)
			{
				UE_LOGF(LogMediaIOCore, Warning, "The capture for %ls will not be cropped because ResizeMethod was already set to \'Resize in Render Pass\' and these options are mutually exclusive.", *MediaCapture->GetMediaOutputName());
			}
			else
			{
				RenderPasses.Add(CreateResamplePass());
			}
		}
		
		if (MediaCapture->GetDesiredCaptureOptions().ColorConversionSettings.IsValid())
		{
			RenderPasses.Add(CreateColorConversionPass(MediaCapture));
		}

		// Final pass
		RenderPasses.Add(CreateFormatConversionPass(MediaCapture));

		return RenderPasses;	
	}

	void FRenderPass::AddConversionPass(const UE::MediaCaptureData::FCaptureFrameArgs& Args, const FConversionPassArgs& ConversionPassArgs, FRDGViewableResource* OutputResource)
	{
		if (OutputResource->Type == FRDGBuffer::StaticType)
		{
			return;
		}

		check(OutputResource->Type == FRDGTexture::StaticType);
		FRDGTexture* OutputTexture =  static_cast<FRDGTexture*>(OutputResource);
		
		//Based on conversion type, this might be changed
		bool bRequiresFormatConversion = ConversionPassArgs.bRequiresFormatConversion;

		// Rectangle area to use from source
		const FIntRect ViewRect(ConversionPassArgs.CopyInfo.GetSourceRect());

		// If no conversion was required, go through a simple copy
		if (Args.MediaCapture->GetConversionOperation() == EMediaCaptureConversionOperation::NONE && !bRequiresFormatConversion)
		{
			AddCopyTexturePass(Args.GraphBuilder, ConversionPassArgs.SourceRGBTexture, OutputTexture, ConversionPassArgs.CopyInfo);
		}
		else
		{
			//At some point we should support color conversion (ocio) but for now we push incoming texture as is
			const bool bDoLinearToSRGB = Args.MediaCapture->GetDesiredCaptureOptions().bApplyLinearToSRGBConversion;

			FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
			TShaderMapRef<FScreenPassVS> VertexShader(GlobalShaderMap);

			const FMatrix& ConversionMatrix = Args.MediaCapture->GetRGBToYUVConversionMatrix();

			switch (Args.MediaCapture->GetConversionOperation())
			{
			case EMediaCaptureConversionOperation::RGBA8_TO_YUV_8BIT:
				{
					//Configure source/output viewport to get the right UV scaling from source texture to output texture
					FScreenPassTextureViewport InputViewport(ConversionPassArgs.SourceRGBTexture, ViewRect);
					FScreenPassTextureViewport OutputViewport(OutputTexture);
					TShaderMapRef<FRGB8toUYVY8ConvertPS> PixelShader(GlobalShaderMap);
					FRGB8toUYVY8ConvertPS::FParameters* Parameters = PixelShader->AllocateAndSetParameters(Args.GraphBuilder, ConversionPassArgs.SourceRGBTexture, ConversionMatrix, MediaShaders::YUVOffset8bits, bDoLinearToSRGB, OutputTexture);
					AddDrawScreenPass(Args.GraphBuilder, RDG_EVENT_NAME("RGBToUYVY 8 bit"), FScreenPassViewInfo(), OutputViewport, InputViewport, VertexShader, PixelShader, Parameters);
				}
				break;
			case EMediaCaptureConversionOperation::RGB10_TO_YUVv210_10BIT:
				{
					//Configure source/output viewport to get the right UV scaling from source texture to output texture
					const FIntPoint InExtent = FIntPoint((((ConversionPassArgs.SourceRGBTexture->Desc.Extent.X + 47) / 48) * 48), ConversionPassArgs.SourceRGBTexture->Desc.Extent.Y);;
					FScreenPassTextureViewport InputViewport(ConversionPassArgs.SourceRGBTexture, ViewRect);
					FScreenPassTextureViewport OutputViewport(OutputTexture);
					TShaderMapRef<FRGB10toYUVv210ConvertPS> PixelShader(GlobalShaderMap);
					FRGB10toYUVv210ConvertPS::FParameters* Parameters = PixelShader->AllocateAndSetParameters(Args.GraphBuilder, ConversionPassArgs.SourceRGBTexture, ConversionMatrix, MediaShaders::YUVOffset10bits, bDoLinearToSRGB, OutputTexture);
					AddDrawScreenPass(Args.GraphBuilder, RDG_EVENT_NAME("RGBToYUVv210"), FScreenPassViewInfo(), OutputViewport, InputViewport, VertexShader, PixelShader, Parameters);
				}
				break;
			case EMediaCaptureConversionOperation::INVERT_ALPHA:
				// fall through
			case EMediaCaptureConversionOperation::SET_ALPHA_ONE:
				// fall through
			case EMediaCaptureConversionOperation::NONE:
				bRequiresFormatConversion = true;
			default:
				if (bRequiresFormatConversion)
				{
					//Configure source/output viewport to get the right UV scaling from source texture to output texture
					FScreenPassTextureViewport InputViewport(ConversionPassArgs.SourceRGBTexture, ViewRect);
					FScreenPassTextureViewport OutputViewport(OutputTexture);

					// In cases where texture is converted from a format that doesn't have A channel, we want to force set it to 1.
					EMediaCaptureConversionOperation MediaConversionOperation = Args.MediaCapture->GetDesiredCaptureOptions().bForceAlphaToOneOnConversion ? EMediaCaptureConversionOperation::SET_ALPHA_ONE : Args.MediaCapture->GetConversionOperation();
					FModifyAlphaSwizzleRgbaPS::FPermutationDomain PermutationVector;
					PermutationVector.Set<FModifyAlphaSwizzleRgbaPS::FConversionOp>(static_cast<int32>(MediaConversionOperation));

					TShaderMapRef<FModifyAlphaSwizzleRgbaPS> PixelShader(GlobalShaderMap, PermutationVector);
					FModifyAlphaSwizzleRgbaPS::FParameters* Parameters = PixelShader->AllocateAndSetParameters(Args.GraphBuilder, ConversionPassArgs.SourceRGBTexture, OutputTexture);
					AddDrawScreenPass(Args.GraphBuilder, RDG_EVENT_NAME("MediaCaptureSwizzle"), FScreenPassViewInfo(), OutputViewport, InputViewport, VertexShader, PixelShader, Parameters);
				}
				break;
			}
		}
	}
	
	void FRenderPass::AddUncompressionPass(const UE::MediaCaptureData::FCaptureFrameArgs& Args, const FBaseRenderPassArgs& UncompressionPassArgs, FRDGViewableResource* OutputResource)
	{
		if (OutputResource->Type == FRDGBuffer::StaticType)
		{
			return;
		}

		check(OutputResource->Type == FRDGTexture::StaticType);
		FRDGTexture* OutputTexture =  static_cast<FRDGTexture*>(OutputResource);

		// Very basic uncompression that draws the compressed texture into a PF_FloatRGBA texture
		AddDrawTexturePass(Args.GraphBuilder, GetGlobalShaderMap(GMaxRHIFeatureLevel), UncompressionPassArgs.SourceRGBTexture, OutputTexture, FRDGDrawTextureInfo());
	}

	void FRenderPass::AddTextureResamplePass(const  UE::MediaCaptureData::FCaptureFrameArgs& Args, const FBaseRenderPassArgs& ConversionPassArgs, FRDGViewableResource* OutputTexture)
	{
		// Downsampling only supports texture at the moment.
		check(OutputTexture->Type == ERDGViewableResourceType::Texture);
			
		const FIntRect ViewRect(ConversionPassArgs.CopyInfo.GetSourceRect());

		RDG_EVENT_SCOPE_STAT(Args.GraphBuilder, MediaCapture_Resample, "MediaCapture_Resample");

		MediaCapture::Resample::AddResamplePass(Args.GraphBuilder, FScreenPassTexture(ConversionPassArgs.SourceRGBTexture), (FRDGTextureRef)OutputTexture);
	}

	FRenderPipeline::FRenderPipeline(UMediaCapture* InMediaCapture)
		: MediaCapture(InMediaCapture)
	{
		RenderPasses = FRenderPass::CreateRenderPasses(InMediaCapture);
	}

	void FRenderPipeline::AddPass(const FRenderPass& InPass, const FName& AfterPass)
	{
		int32 Index = RenderPasses.IndexOfByPredicate([&AfterPass](const FRenderPass& RenderPass)
		{
			return RenderPass.Name == AfterPass;
		});
		
		RenderPasses.Insert(InPass, Index + 1);
	}

	void FRenderPipeline::InitializeResources_RenderThread(const TSharedPtr<MediaCaptureData::FCaptureFrame>& CapturingFrame, FRDGBuilder& GraphBuilder, FRDGTextureRef InputResource) const
	{
		if (CapturingFrame->bMediaCaptureActive)
		{
			FInitializePassOutputArgs InitializePassOutputArgs;
			InitializePassOutputArgs.MediaCapture = MediaCapture;
			InitializePassOutputArgs.InputResource = InputResource;

			UE::MediaCapture::FRenderPassFrameResources RenderPassResources;
				
			for (const FRenderPass& RenderPass : RenderPasses)
			{
				// This will only initialize on the first call.
				InitializePassOutputResource(RenderPassResources, CapturingFrame->FrameId, InitializePassOutputArgs, RenderPass);
				FRDGViewableResource* OutputResource = RegisterPassOutputResource(RenderPassResources, GraphBuilder, RenderPass);

				check(OutputResource);

				// Each pass output resource will be passed to the next pass's input for initialisation in case they want to copy the previous resource's description.
				InitializePassOutputArgs.InputResource = OutputResource;
			}

			RenderPassResources.LastRenderPassName = RenderPasses.Last().Name;
			CapturingFrame->RenderPassResources = MoveTemp(RenderPassResources);
		}
	}

	FRDGViewableResource* FRenderPipeline::ExecutePasses_RenderThread(const UE::MediaCaptureData::FCaptureFrameArgs& Args, const TSharedPtr<MediaCaptureData::FCaptureFrame>& CapturingFrame, FRDGBuilder& GraphBuilder, FRDGTextureRef InputRGBTexture) const
	{
		FRDGViewableResource* PreviousOutput = InputRGBTexture;

		if (CapturingFrame->bMediaCaptureActive)
		{
			for (const UE::MediaCapture::FRenderPass& RenderPass : RenderPasses)
			{
				// Validate if we have a resources used to capture source texture
				if (!CapturingFrame->HasValidResource())
				{
					UE_LOGF(LogMediaIOCore, Error, "The capture will stop for '%ls'. A capture frame had an invalid render resource.", *MediaCapture->GetMediaOutputName());
					return PreviousOutput;
				}


				// Register pass rdg resource
				bool bExecutePass = true;
				if (RenderPass.ShouldExecutePassDelegate.IsBound())
				{
					bExecutePass = RenderPass.ShouldExecutePassDelegate.Execute(Args, PreviousOutput);
				}

				if (bExecutePass)
				{
					FRDGViewableResource* PassOutput = RegisterPassOutputResource(CapturingFrame->RenderPassResources, GraphBuilder, RenderPass);
					RenderPass.ExecutePassDelegate.Execute(Args, CapturingFrame, PreviousOutput, PassOutput);
					PreviousOutput = PassOutput;
				}
			}

		}
		return PreviousOutput;
	}

	void UE::MediaCapture::FRenderPipeline::InitializePassOutputResource(UE::MediaCapture::FRenderPassFrameResources& InOutFrameResources, int32 InFrameId, FInitializePassOutputArgs InArgs, const FRenderPass& InRenderPass)
	{
		if (InRenderPass.InitializePassOutputDelegate.IsBound())
		{
			if (InRenderPass.OutputType == ERDGViewableResourceType::Texture && !InOutFrameResources.PassTextureOutputs.Contains(InRenderPass.Name))
			{
				TRefCountPtr<IPooledRenderTarget> RenderTarget = InRenderPass.InitializePassOutputDelegate.Execute(InArgs, InFrameId).Get<TRefCountPtr<IPooledRenderTarget>>();
				InOutFrameResources.PassTextureOutputs.Add(InRenderPass.Name, MoveTemp(RenderTarget));
			}
			else if (InRenderPass.OutputType == ERDGViewableResourceType::Buffer && !InOutFrameResources.PassBufferOutputs.Contains(InRenderPass.Name))
			{
				TRefCountPtr<FRDGPooledBuffer> PooledBuffer = InRenderPass.InitializePassOutputDelegate.Execute(InArgs, InFrameId).Get<TRefCountPtr<FRDGPooledBuffer>>();
				InOutFrameResources.PassBufferOutputs.Add(InRenderPass.Name, MoveTemp(PooledBuffer));
			}
		}
	}

	FRDGViewableResource* UE::MediaCapture::FRenderPipeline::RegisterPassOutputResource(const UE::MediaCapture::FRenderPassFrameResources& InFrameResources, FRDGBuilder& InRDGBuilder, const UE::MediaCapture::FRenderPass& InRenderPass)
	{
		if (InRenderPass.OutputType == ERDGViewableResourceType::Texture)
		{
			const TRefCountPtr<IPooledRenderTarget>& RT = InFrameResources.PassTextureOutputs[InRenderPass.Name];
			return InRDGBuilder.RegisterExternalTexture(RT, RT->GetDesc().DebugName);
		}
		else
		{
			const TRefCountPtr<FRDGPooledBuffer>& Buffer = InFrameResources.PassBufferOutputs[InRenderPass.Name];
			return InRDGBuilder.RegisterExternalBuffer(Buffer, Buffer->GetName());
		}
	}
}
