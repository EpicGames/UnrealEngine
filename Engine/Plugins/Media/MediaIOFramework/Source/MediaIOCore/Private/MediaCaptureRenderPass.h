// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "MediaShaders.h"
#include "Misc/TVariant.h"
#include "OpenColorIOColorSpace.h"
#include "OpenColorIORendering.h"
#include "RendererInterface.h"
#include "RenderGraphResources.h"
#include "RenderGraphUtils.h"
#include "UObject/WeakObjectPtr.h"

class UMediaCapture;

namespace UE::MediaCaptureData
{
	struct FCaptureFrameArgs;
	class FCaptureFrame;
}
	
/** Contains the most basic arguments needed for a render pass. */
struct FBaseRenderPassArgs
{
	FRDGTextureRef SourceRGBTexture;
	FRHICopyTextureInfo CopyInfo;
};

/** Contains arguments for the color conversion pass. */ 
struct FColorConversionPassArgs : public FBaseRenderPassArgs
{
	FOpenColorIORenderPassResources OCIOResources;
};

/** Helper struct to contain arguments for AddConversionPass */
struct FConversionPassArgs : public FBaseRenderPassArgs
{
	bool bRequiresFormatConversion = false;
	FVector2D SizeU = FVector2D::ZeroVector;
	FVector2D SizeV = FVector2D::ZeroVector;
};

namespace UE::MediaCapture
{
	/** Parameters for calling the Initialize Pass Output callback. */
	struct FInitializePassOutputArgs
	{
		const UMediaCapture* MediaCapture = nullptr;
		FRDGViewableResource* InputResource = nullptr;
	};

	/**
	 * Describes a render pass that will be applied to the captured texture before it's copied to the final output buffer. 
	 * Every render pass must have a valid InitializePassOutputDelegate and ExecutePassDelegate.
	 * The InitializePassOutput delegate will be called on the render thread when we capture the first frame.
	 * The ExecutePassDelegate will be called on the render thread in the order defined in the media capture's render pipeline.
	 */
	struct FRenderPass
	{
		using TRDGResource = TVariant<TRefCountPtr<IPooledRenderTarget>, TRefCountPtr<FRDGPooledBuffer>>;

		/** Create media capture render passes based on media capture settings. */
		static TArray<FRenderPass> CreateRenderPasses(const UMediaCapture* MediaCapture);
		/** Add a texture Resample/upsample pass to the graph builder. */
		static void AddTextureResamplePass(const  UE::MediaCaptureData::FCaptureFrameArgs& Args, const FBaseRenderPassArgs& ConversionPassArgs, FRDGViewableResource* OutputTexture);
		/** Add a color conversion pass to the graph builder. */
		static void AddColorConversionPass(const UE::MediaCaptureData::FCaptureFrameArgs& Args, const TSharedPtr<UE::MediaCaptureData::FCaptureFrame> CapturingFrame, const FColorConversionPassArgs& ConversionPassArgs, FRDGViewableResource* OutputResource);
		/** Add a format conversion pass to the graph builder. */
		static void AddConversionPass(const UE::MediaCaptureData::FCaptureFrameArgs& Args, const FConversionPassArgs& ConversionPassArgs, FRDGViewableResource* OutputResource);
		/** Add a pass to uncompress a compressed texture into a proper uncompressed format (currently PF_FloatRGBA) */
		static void AddUncompressionPass(const UE::MediaCaptureData::FCaptureFrameArgs& Args, const FBaseRenderPassArgs& UncompressionPassArgs, FRDGViewableResource* OutputResource);

		/** This delegate should return a TRDGResource constructed with either a pooled texture or buffer. */
		DECLARE_DELEGATE_RetVal_TwoParams(TRDGResource /*Output*/, FInitializePassOutput, const UE::MediaCapture::FInitializePassOutputArgs& /*InitializePassOutputArgs*/, uint32 /*FrameId*/);
		/**
		 * Method called to initialize the pass output texture in a given capturing frame.
		 * @Note: Will be called on the render thread.
		 */
		FInitializePassOutput InitializePassOutputDelegate;

		DECLARE_DELEGATE_FourParams(FExecutePass, const UE::MediaCaptureData::FCaptureFrameArgs& /*Args*/, const TSharedPtr<UE::MediaCaptureData::FCaptureFrame> /*CapturingFrame*/, FRDGViewableResource* /*InputResource*/, FRDGViewableResource* /*Output*/);
		/**
		 * Method called to execute the pass.
		 * @Note: Will be called on the render thread.
		 */
		FExecutePass ExecutePassDelegate;

		/**
		 * Delegate invoked to query whether the pass should be executed.
		 */
		DECLARE_DELEGATE_RetVal_TwoParams(bool, FShouldExecutePass, const UE::MediaCaptureData::FCaptureFrameArgs& /*Args*/, FRDGViewableResource* /*InputResource*/);
		FShouldExecutePass ShouldExecutePassDelegate;

		/** Name of the pass. */
		FName Name;

		/** Whether the output of the pass is a texture or a buffer. */
		ERDGViewableResourceType OutputType = ERDGViewableResourceType::Texture;

	private:
		/** Creates a color conversion pass. */
		static FRenderPass CreateColorConversionPass(const UMediaCapture* MediaCapture);
		/** Creates a resampling pass. */
		static FRenderPass CreateResamplePass();
		/** Creates a format conversion pass according to the capture settings. */
		static FRenderPass CreateFormatConversionPass(const UMediaCapture* MediaCapture);
		/** Creates an uncompression pass according to the capture settings. */
		static FRenderPass CreateUncompressionPass(const UMediaCapture* MediaCapture);
	};

	/** Instantiates a TRDGResource TVariant from a pooled render target. */ 
	static FRenderPass::TRDGResource RenderTargetResource(TRefCountPtr<IPooledRenderTarget>&& RenderTarget)
	{
		return FRenderPass::TRDGResource(TInPlaceType<TRefCountPtr<IPooledRenderTarget>>(), MoveTemp(RenderTarget));
	}

	/** Instantiates a TRDGResource TVariant from a pooled buffer. */
	static FRenderPass::TRDGResource BufferResource(TRefCountPtr<FRDGPooledBuffer>&& Buffer)
	{
		return FRenderPass::TRDGResource(TInPlaceType<TRefCountPtr<FRDGPooledBuffer>>(), MoveTemp(Buffer));
	}

	/** Creates the necessary render target for the color conversion pass. */
	static FRenderPass::TRDGResource InitializeColorConversionPassOutputTexture(const FInitializePassOutputArgs& Args, uint32 FrameId);

	/** Creates the necessary render target for the Resample pass. */
	static FRenderPass::TRDGResource InitializeResamplePassOutputTexture(const FInitializePassOutputArgs& Args, uint32 FrameId);

	/** Data related to render passes that live on a capturing frame. **/
	struct FRenderPassFrameResources
	{
		/** Get the render target used by the last render pass in the pipeline. */
		TRefCountPtr<IPooledRenderTarget> GetLastRenderTarget() const
		{
			return PassTextureOutputs.FindRef(LastRenderPassName);
		}

		/** Get the buffer used by the last render pass in the pipeline. */
		TRefCountPtr<FRDGPooledBuffer> GetLastBuffer() const
		{
			return PassBufferOutputs.FindRef(LastRenderPassName);
		}

		/** Register and retrieve the final resource used by this capture frame in the render passes. */
		FRDGViewableResource* GetFinalRDGResource(FRDGBuilder& GraphBuilder) const
		{
			if (const TRefCountPtr<IPooledRenderTarget> FinalRT = GetLastRenderTarget())
			{
				return GraphBuilder.RegisterExternalTexture(FinalRT, FinalRT->GetDesc().DebugName);
			}
			else if (const TRefCountPtr<FRDGPooledBuffer> FinalBuffer = GetLastBuffer())
			{
				return GraphBuilder.RegisterExternalBuffer(FinalBuffer, FinalBuffer->GetName());
			}
			return nullptr;
		}

		/** Holds the name of the last render pass. */
		FName LastRenderPassName;
		/** Map of pass name to output render target. */
		TMap<FName, TRefCountPtr<IPooledRenderTarget>> PassTextureOutputs;
		/** Map of pass name to output buffer. */
		TMap<FName, TRefCountPtr<FRDGPooledBuffer>> PassBufferOutputs;
	};

	/**
	 * Media Capture render pipeline.
	 * Holds and initializes render passes based on a media capture's settings.
	 */
	class FRenderPipeline
	{
	public:
		FRenderPipeline(UMediaCapture* InMediaCapture);

		/** Adds a render pass to the media capture render pipeline, optionally positioning it after the pass with the specified name */
		void AddPass(const FRenderPass& InPass, const FName& AfterPass = NAME_None);
		
		/** Initialize and register render pass resources for a given capture frame. */
		void InitializeResources_RenderThread(const TSharedPtr<MediaCaptureData::FCaptureFrame>& CapturingFrame, FRDGBuilder& GraphBuilder, FRDGTextureRef InputResource) const;

		/** 
		 * Execute passes in the pipeline.
		 * @return The final output generated by the render pipeline.
		 **/
		FRDGViewableResource* ExecutePasses_RenderThread(const UE::MediaCaptureData::FCaptureFrameArgs& Args, const TSharedPtr<MediaCaptureData::FCaptureFrame>& CapturingFrame, FRDGBuilder& GraphBuilder, FRDGTextureRef InputRGBTexture) const;

		/** Returns the name of the last render pass. */
		FName GetLastPassName() const
		{
			if (RenderPasses.Num())
			{
				return RenderPasses.Last().Name;
			}
			return NAME_None;
		}

	private:
		/** Initialize render pass resources for a given capture frame. */
		static void InitializePassOutputResource(UE::MediaCapture::FRenderPassFrameResources& InOutFrameResources, int32 InFrameId, FInitializePassOutputArgs InArgs, const FRenderPass& InRenderPass);
		/** Register a render pass' resources with the render graph builder. Calling this multiple times will simply return the (already) registered resource. */
		static FRDGViewableResource* RegisterPassOutputResource(const UE::MediaCapture::FRenderPassFrameResources& InFrameResources, FRDGBuilder& InRDGBuilder, const UE::MediaCapture::FRenderPass& InRenderPass);
	private:
		UMediaCapture* MediaCapture = nullptr;
		TArray<FRenderPass> RenderPasses;
	};

}