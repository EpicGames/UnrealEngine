// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneViewExtension.h"
#include "MediaCapture.h"
#include "MediaIOCoreModule.h"
#include "ImagePixelData.h"
#include "PostProcess/PostProcessMaterialInputs.h"
#include "RenderGraphUtils.h"
#include "RHI.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "SceneView.h"
#include "ScreenPass.h"
#include "RenderUtils.h"


class FRDGTexture;

/**
 * View extension that calls into MediaCapture to capture frames as soon as the frame is rendered.
 */
class FMediaCaptureSceneViewExtension : public FSceneViewExtensionBase
{
public:
	
	FMediaCaptureSceneViewExtension(const FAutoRegister& InAutoRegister, UMediaCapture* InMediaCapture, EMediaCapturePhase InCapturePhase, int32 InPriority, FRenderTarget* InRenderTarget)
		: FSceneViewExtensionBase(InAutoRegister)
		, WeakCapture(InMediaCapture)
		, CapturePhase(InCapturePhase)
		, Priority(InPriority)
		, CurrentRenderTarget(InRenderTarget)
	{
	}

	//~ Begin FSceneViewExtensionBase Interface
	virtual int32 GetPriority() const override
	{
		return Priority;
	}

	virtual void SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView) override 
	{
		if (bValidPhase && CurrentRenderTarget == InView.Family->RenderTarget)
		{
			// Copied from PostProcessing.h
			if (InView.GetFeatureLevel() >= ERHIFeatureLevel::SM5)
			{
				bPostProcessingEnabled =
					InView.Family->EngineShowFlags.PostProcessing &&
					!InView.Family->EngineShowFlags.VisualizeDistanceFieldAO &&
					!InView.Family->EngineShowFlags.VisualizeShadingModels &&
					!InView.Family->EngineShowFlags.VisualizeVolumetricCloudConservativeDensity &&
					!InView.Family->EngineShowFlags.VisualizeVolumetricCloudEmptySpaceSkipping &&
					!InView.Family->EngineShowFlags.ShaderComplexity;
			}
			else
			{
				bPostProcessingEnabled = InView.Family->EngineShowFlags.PostProcessing && !InView.Family->EngineShowFlags.ShaderComplexity && IsMobileHDR();
			}

			if (CapturePhase != EMediaCapturePhase::BeforePostProcessing && CapturePhase != EMediaCapturePhase::EndFrame && CapturePhase != EMediaCapturePhase::PostRender)
			{
				if (!bPostProcessingEnabled)
				{
					LastErrorMessage = TEXT("Media Capture will not work since it is scheduled in a post processing phase and post processing is not enabled.");
					UE_LOGF(LogMediaIOCore, Warning, "%ls", *LastErrorMessage);
					bValidPhase = false;
				}
			}
		}
	};

	virtual void SubscribeToPostProcessingPass(EPostProcessingPass PassId, const FSceneView& InView, FAfterPassCallbackDelegateArray& InOutPassCallbacks, bool bIsPassEnabled) override
	{
		if (CurrentRenderTarget == InView.Family->RenderTarget)
		{
			if ((CapturePhase == EMediaCapturePhase::AfterMotionBlur && PassId == EPostProcessingPass::MotionBlur)
				|| (CapturePhase == EMediaCapturePhase::AfterToneMap && PassId == EPostProcessingPass::Tonemap)
				|| (CapturePhase == EMediaCapturePhase::AfterFXAA && PassId == EPostProcessingPass::FXAA)
				|| (CapturePhase == EMediaCapturePhase::BeforePostProcessing && PassId == EPostProcessingPass::SSRInput))
			{
				InOutPassCallbacks.Add(FAfterPassCallbackDelegate::CreateSP(this, &FMediaCaptureSceneViewExtension::PostProcessCallback_RenderThread));
			}
		}
	}

	virtual bool IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const override
	{
		return true;
	}

	virtual void PostRenderView_RenderThread(FRDGBuilder& GraphBuilder, FSceneView& InView) override
	{
		if (CapturePhase == EMediaCapturePhase::PostRender && CurrentRenderTarget == InView.Family->RenderTarget)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MediaCaptureExtensionCallback);

			FTextureRHIRef RenderTarget = InView.Family->RenderTarget->GetRenderTargetTexture();

			RDG_EVENT_SCOPE(GraphBuilder, "MediaCaptureSceneExtension");
			if (WeakCapture.IsValid())
			{
				WeakCapture->TryCaptureImmediate_RenderThread(GraphBuilder, RenderTarget, InView.UnscaledViewRect);
			}
		}
	}
	//~ End FSceneViewExtensionBase Interface

	FScreenPassTexture PostProcessCallback_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessMaterialInputs& InOutInputs)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MediaCaptureExtensionCallback);

		FScreenPassTexture SceneColor = FScreenPassTexture::CopyFromSlice(GraphBuilder, InOutInputs.GetInput(EPostProcessMaterialInput::SceneColor));
		check(SceneColor.IsValid());

		if (FRDGTextureRef TextureRef = SceneColor.Texture)
		{
			RDG_EVENT_SCOPE(GraphBuilder, "MediaCaptureSceneExtension");
			if (WeakCapture.IsValid())
			{
				bool bCaptureSucceeded = WeakCapture->TryCaptureImmediate_RenderThread(GraphBuilder, TextureRef, SceneColor.ViewRect);
				if(!bCaptureSucceeded)
				{
					LastErrorMessage = TEXT("Failed to capture resource.");
					UE_LOGF(LogMediaIOCore, VeryVerbose, "%ls", *LastErrorMessage);
				}
			}
		}

		return InOutInputs.ReturnUntouchedSceneColorForPostProcessing(GraphBuilder);
	}

	bool IsValid() const
	{
		return bValidPhase;
	}

private:
	TWeakObjectPtr<UMediaCapture> WeakCapture;
	EMediaCapturePhase CapturePhase = EMediaCapturePhase::AfterMotionBlur;
	bool bPostProcessingEnabled = true;
	bool bValidPhase = true;
	FString LastErrorMessage;
	int32 Priority = 0;
	/** Render target that should be considered by this scene view extension. */
	FRenderTarget* CurrentRenderTarget = nullptr;
};

