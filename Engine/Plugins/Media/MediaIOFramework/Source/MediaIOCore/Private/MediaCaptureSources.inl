// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/RendererSettings.h"
#include "Framework/Application/SlateApplication.h"
#include "Math/IntPoint.h"
#include "MediaCapture.h"
#include "MediaIOCoreModule.h"
#include "MediaOutput.h"
#include "MediaTexture.h"
#include "PixelShaderUtils.h"
#include "Slate/SceneViewport.h"
#include "TextureResource.h"
#include "UObject/WeakObjectPtr.h"
#include "ViewportClient.h"

namespace UE::MediaCapture::Private
{
	bool ValidateSize(const FIntPoint TargetSize, const FIntPoint& DesiredSize, const FMediaCaptureOptions& CaptureOptions, const bool bCurrentlyCapturing)
	{
		if (CaptureOptions.Crop == EMediaCaptureCroppingType::None)
		{
			if (DesiredSize.X != TargetSize.X || DesiredSize.Y != TargetSize.Y)
			{
				UE_LOGF(LogMediaIOCore, Error, "Can not %ls the capture. The Render Target size doesn't match with the requested size. SceneViewport: %d,%d  MediaOutput: %d,%d"
					, bCurrentlyCapturing ? TEXT("continue") : TEXT("start")
					, TargetSize.X, TargetSize.Y
					, DesiredSize.X, DesiredSize.Y);
				return false;
			}
		}
		else
		{
			FIntPoint StartCapturePoint = FIntPoint::ZeroValue;
			if (CaptureOptions.Crop == EMediaCaptureCroppingType::Custom)
			{
				if (CaptureOptions.CustomCapturePoint.X < 0 || CaptureOptions.CustomCapturePoint.Y < 0)
				{
					UE_LOGF(LogMediaIOCore, Error, "Can not %ls the capture. The start capture point is negatif. Start Point: %d,%d"
						, bCurrentlyCapturing ? TEXT("continue") : TEXT("start")
						, StartCapturePoint.X, StartCapturePoint.Y);
					return false;
				}
				StartCapturePoint = CaptureOptions.CustomCapturePoint;
			}

			if (DesiredSize.X + StartCapturePoint.X > TargetSize.X || DesiredSize.Y + StartCapturePoint.Y > TargetSize.Y)
			{
				UE_LOGF(LogMediaIOCore, Error, "Can not %ls the capture. The Render Target size is too small for the requested cropping options. SceneViewport: %d,%d  MediaOutput: %d,%d Start Point: %d,%d"
					, bCurrentlyCapturing ? TEXT("continue") : TEXT("start")
					, TargetSize.X, TargetSize.Y
					, DesiredSize.X, DesiredSize.Y
					, StartCapturePoint.X, StartCapturePoint.Y);
				return false;
			}
		}

		return true;
	}

	bool ValidateSceneViewport(const TSharedPtr<FSceneViewport>& SceneViewport, const FMediaCaptureOptions& CaptureOptions, const FIntPoint& DesiredSize, const EPixelFormat DesiredPixelFormat, const bool bCurrentlyCapturing)
	{
		if (!SceneViewport.IsValid())
		{
			UE_LOGF(LogMediaIOCore, Error, "Can not %ls the capture. The Scene Viewport is invalid."
				, bCurrentlyCapturing ? TEXT("continue") : TEXT("start"));
			return false;
		}

		const FIntPoint SceneViewportSize = SceneViewport->GetRenderTargetTextureSizeXY();
		if (CaptureOptions.ResizeMethod != EMediaCaptureResizeMethod::ResizeInRenderPass && !ValidateSize(SceneViewportSize, DesiredSize, CaptureOptions, bCurrentlyCapturing))
		{
			return false;
		}

		static const auto CVarDefaultBackBufferPixelFormat = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.DefaultBackBufferPixelFormat"));
		EPixelFormat SceneTargetFormat = EDefaultBackBufferPixelFormat::Convert2PixelFormat(EDefaultBackBufferPixelFormat::FromInt(CVarDefaultBackBufferPixelFormat->GetValueOnGameThread()));
		if (DesiredPixelFormat != SceneTargetFormat)
		{
			if (!UMediaCapture::GetSupportedRgbaSwizzleFormats().Contains(SceneTargetFormat) || !CaptureOptions.bConvertToDesiredPixelFormat)
			{
				UE_LOGF(LogMediaIOCore, Error, "Can not %ls the capture. The Render Target pixel format doesn't match with the requested pixel format. %lsRenderTarget: %ls MediaOutput: %ls"
					, bCurrentlyCapturing ? TEXT("continue") : TEXT("start")
					, (UMediaCapture::GetSupportedRgbaSwizzleFormats().Contains(SceneTargetFormat) && !CaptureOptions.bConvertToDesiredPixelFormat) ? TEXT("Please enable \"Convert To Desired Pixel Format\" option in Media Capture settings") : TEXT("")
					, GetPixelFormatString(SceneTargetFormat)
					, GetPixelFormatString(DesiredPixelFormat));
				return false;
			}
			else
			{
				UE_LOGF(LogMediaIOCore, Warning, "The Render Target pixel format doesn't match with the requested pixel format. Render target will be automatically converted. This could have a slight performance impact. RenderTarget: %ls MediaOutput: %ls"
					, GetPixelFormatString(SceneTargetFormat)
					, GetPixelFormatString(DesiredPixelFormat));
			}
		}

		return true;
	}
	
	bool ValidateTextureRenderTarget2D(const UTextureRenderTarget2D* InRenderTarget2D, const FMediaCaptureOptions& CaptureOptions, const FIntPoint& DesiredSize, const EPixelFormat DesiredPixelFormat, const bool bCurrentlyCapturing)
	{
		if (InRenderTarget2D == nullptr)
		{
			UE_LOGF(LogMediaIOCore, Error, "Couldn't %ls the capture. The Render Target is invalid."
				, bCurrentlyCapturing ? TEXT("continue") : TEXT("start"));
			return false;
		}

		if (CaptureOptions.ResizeMethod != EMediaCaptureResizeMethod::ResizeInRenderPass && !ValidateSize(FIntPoint(InRenderTarget2D->SizeX, InRenderTarget2D->SizeY), DesiredSize, CaptureOptions, bCurrentlyCapturing))
		{
			return false;
		}

		if (DesiredPixelFormat != InRenderTarget2D->GetFormat())
		{
			if (!UMediaCapture::GetSupportedRgbaSwizzleFormats().Contains(InRenderTarget2D->GetFormat()) || !CaptureOptions.bConvertToDesiredPixelFormat)
			{
				UE_LOGF(LogMediaIOCore, Error, "Can not %ls the capture. The Render Target pixel format doesn't match with the requested pixel format. %lsRenderTarget: %ls MediaOutput: %ls"
					, bCurrentlyCapturing ? TEXT("continue") : TEXT("start")
					, (UMediaCapture::GetSupportedRgbaSwizzleFormats().Contains(InRenderTarget2D->GetFormat()) && !CaptureOptions.bConvertToDesiredPixelFormat) ? TEXT("Please enable \"Convert To Desired Pixel Format\" option in Media Capture settings. ") : TEXT("")
					, GetPixelFormatString(InRenderTarget2D->GetFormat())
					, GetPixelFormatString(DesiredPixelFormat));
				return false;
			}
			else
			{
				UE_LOGF(LogMediaIOCore, Warning, "The Render Target pixel format doesn't match with the requested pixel format. Render target will be automatically converted. This could have a slight performance impact. RenderTarget: %ls MediaOutput: %ls"
					, GetPixelFormatString(InRenderTarget2D->GetFormat())
					, GetPixelFormatString(DesiredPixelFormat));
			}
		}

		return true;
	}

	bool ValidateMediaTexture(const UMediaTexture* InMediaTexture, const FMediaCaptureOptions& CaptureOptions, const FIntPoint& TransformedMediaTextureSize, const FIntPoint& DesiredSize, const EPixelFormat DesiredPixelFormat, const bool bCurrentlyCapturing)
	{
		if (InMediaTexture == nullptr)
		{
			UE_LOGF(LogMediaIOCore, Error, "Couldn't %ls the capture. The Media Texture is invalid."
				, bCurrentlyCapturing ? TEXT("continue") : TEXT("start"));
			return false;
		}

		if (CaptureOptions.ResizeMethod != EMediaCaptureResizeMethod::ResizeInRenderPass && !ValidateSize(TransformedMediaTextureSize, DesiredSize, CaptureOptions, bCurrentlyCapturing))
		{
			return false;
		}

		return true;
	}

	enum class ECaptureType : uint8
	{
		Immediate,
		OnTick
	};
	
	class FCaptureSource
	{
	public:
		FCaptureSource(UMediaCapture* InMediaCapture, FMediaCaptureOptions InCaptureOptions)
			: MediaCapture(InMediaCapture)
			, CaptureOptions(MoveTemp(InCaptureOptions))
		{
		}

		virtual ~FCaptureSource() = default;
		
		virtual void ResizeSourceBuffer(FIntPoint Size) {}
		virtual void ResetSourceBufferSize(bool bFlushRenderingCommands) {}
		virtual void ApplyShowFlagChanges() {}
		virtual void RestoreShowFlags() {}
		virtual bool ValidateSource(const FIntPoint& DesiredSize, const EPixelFormat DesiredPixelFormat, const bool bCurrentlyCapturing) = 0;
		virtual bool PostInitialize() = 0;
		virtual EMediaCaptureSourceType GetSourceType() = 0;
		virtual FIntPoint GetSize() = 0;
		/** Returns whether the underlying captured resource is still valid. */
		virtual bool IsValid() const = 0;
		
		virtual ECaptureType GetCaptureType() const
		{
			return ECaptureType::OnTick;
		}
		
		virtual bool UpdateSourceImpl()
		{
			return true;
		}
		
		/** Only returns a valid texture for scene viewports and render targets as the RHI Sources don't keep a pointer to the RHI texture. */
		virtual FTextureRHIRef GetSourceTextureForInput_RenderThread()
		{
			return nullptr;
		}

		virtual UTextureRenderTarget2D* GetRenderTarget() const
		{
			return nullptr;
		}
		
		virtual TSharedPtr<FSceneViewport> GetSceneViewport() const
		{
			return nullptr;
		}

		/** Allows the capture source to add its own render passes to the media capture render pipeline */
		virtual void AddPasses(FRenderPipeline& Pipeline) { }

		virtual TSharedPtr<SWindow> GetTargetWindow() const
		{
			return nullptr;
		}

	public:
		TWeakObjectPtr<UMediaCapture> MediaCapture;
		FMediaCaptureOptions CaptureOptions;
	};

	class FSceneViewportCaptureSource : public FCaptureSource
	{
	public:
		FSceneViewportCaptureSource(UMediaCapture* InMediaCapture, FMediaCaptureOptions InCaptureOptions, TSharedPtr<FSceneViewport> InSceneViewport)
			: FCaptureSource(InMediaCapture, MoveTemp(InCaptureOptions))
			, WeakViewport(InSceneViewport)
		{
			if (TSharedPtr<SViewport> ViewportWidget = InSceneViewport->GetViewportWidget().Pin())
			{
				TargetWindow = FSlateApplication::Get().FindWidgetWindow(ViewportWidget.ToSharedRef());
			}
		}

		virtual ECaptureType GetCaptureType() const override
		{
			if (CaptureOptions.CapturePhase != EMediaCapturePhase::EndFrame)
            {
				return ECaptureType::Immediate;
            }
			return ECaptureType::OnTick;
		}
		
		virtual EMediaCaptureSourceType GetSourceType() override
		{
			return EMediaCaptureSourceType::SCENE_VIEWPORT;
		}
		
		virtual void ResetSourceBufferSize(bool bFlushRenderingCommands) override
		{
			if (TSharedPtr<FSceneViewport> SceneViewport = WeakViewport.Pin())
			{
				if (MediaCapture.IsValid())
				{
					MediaCapture->ResetFixedViewportSize(SceneViewport, bFlushRenderingCommands);
				}
			}
		}

#if WITH_EDITOR
		virtual void ApplyShowFlagChanges() override
		{
			if (!CaptureOptions.bDisableEditorSpritesDuringCapture)
			{
				return;
			}

			if (TSharedPtr<FSceneViewport> SceneViewport = WeakViewport.Pin())
			{
				if (FEngineShowFlags* ShowFlags = SceneViewport->GetClient()->GetEngineShowFlags())
				{
					bOriginalBillboardSprites = ShowFlags->BillboardSprites != 0;
					ShowFlags->SetBillboardSprites(false);
				}
			}
		}

		virtual void RestoreShowFlags() override
		{
			if (!CaptureOptions.bDisableEditorSpritesDuringCapture)
			{
				return;
			}

			if (TSharedPtr<FSceneViewport> SceneViewport = WeakViewport.Pin())
			{
				if (FEngineShowFlags* ShowFlags = SceneViewport->GetClient()->GetEngineShowFlags())
				{
					ShowFlags->SetBillboardSprites(bOriginalBillboardSprites);
				}
			}
		}
#endif

		virtual void ResizeSourceBuffer(FIntPoint Size) override
		{
			if (TSharedPtr<FSceneViewport> SceneViewport = WeakViewport.Pin())
			{
				if (MediaCapture.IsValid())
				{
					MediaCapture->SetFixedViewportSize(SceneViewport, Size);
				}
			}
		}

		virtual bool ValidateSource(const FIntPoint& InDesiredSize, const EPixelFormat InDesiredPixelFormat, const bool bInCurrentlyCapturing) override
		{
			bool bResult = false;
			if (TSharedPtr<FSceneViewport> SceneViewport = WeakViewport.Pin())
			{
				bResult = ValidateSceneViewport(SceneViewport, CaptureOptions, InDesiredSize, InDesiredPixelFormat, bInCurrentlyCapturing);
				if (!bResult)
				{
					MediaCapture->ResetFixedViewportSize(SceneViewport, false);
				}
			}

			return bResult;
		}

		virtual bool PostInitialize() override
		{
			if (TSharedPtr<FSceneViewport> SceneViewport = WeakViewport.Pin())
			{
				return MediaCapture->PostInitializeCaptureViewport(SceneViewport);
			}
			return false;
		}
		
		virtual FIntPoint GetSize() override
		{
			if (TSharedPtr<FSceneViewport> SceneViewport = WeakViewport.Pin())
			{
				return SceneViewport->GetSize();
			}
			return FIntPoint();
		}

		virtual bool UpdateSourceImpl() override
		{
			if (MediaCapture.IsValid())
			{
				if (TSharedPtr<FSceneViewport> SceneViewport = WeakViewport.Pin())
				{
					return MediaCapture->UpdateSceneViewportImpl(SceneViewport);
				}
			}

			return false;
		}

		virtual FTextureRHIRef GetSourceTextureForInput_RenderThread() override
		{
			FTextureRHIRef SourceTexture;
			if (TSharedPtr<FSceneViewport> Viewport = WeakViewport.Pin())
			{
				SourceTexture = Viewport->GetRenderTargetTexture();
			}

			return SourceTexture;
		}

		virtual TSharedPtr<FSceneViewport> GetSceneViewport() const override
		{
			return WeakViewport.Pin();
		}

		virtual bool IsValid() const override
		{
			return WeakViewport.IsValid();
		}

		virtual TSharedPtr<SWindow> GetTargetWindow() const override
		{
			return TargetWindow.Pin();
		}
		
		TWeakPtr<FSceneViewport> WeakViewport;
		TWeakPtr<SWindow> TargetWindow;
#if WITH_EDITOR
		bool bOriginalBillboardSprites = true;
#endif
	};

	class FRHIResourceCaptureSource : public FCaptureSource
	{
	public:
		FRHIResourceCaptureSource(UMediaCapture* InMediaCapture, FMediaCaptureOptions InCaptureOptions, const FRHICaptureResourceDescription& InResourceDescription)
			: FCaptureSource(InMediaCapture, MoveTemp(InCaptureOptions))
			, ResourceDescription(InResourceDescription)
		{
		}

		virtual ECaptureType GetCaptureType() const override
		{
			return ECaptureType::Immediate;
		}

		virtual EMediaCaptureSourceType GetSourceType() override
		{
			return EMediaCaptureSourceType::RHI_RESOURCE;
		}

		virtual bool PostInitialize() override
		{
			return MediaCapture->PostInitializeCaptureRHIResource(ResourceDescription);
		}

		virtual FIntPoint GetSize() override
		{
			return ResourceDescription.ResourceSize;
		}

		virtual bool ValidateSource(const FIntPoint& DesiredSize, const EPixelFormat DesiredPixelFormat, const bool bCurrentlyCapturing) override
		{
			return true;
		}

		virtual bool IsValid() const override
		{
			// We don't keep a handle to the resource, and it may change on every frame so we always return true to prevent stopping the capture.
			return true;
		}

		FRHICaptureResourceDescription ResourceDescription;
	};


	class FRenderTargetCaptureSource : public FCaptureSource
	{
	public:
		FRenderTargetCaptureSource(UMediaCapture* InMediaCapture, FMediaCaptureOptions InCaptureOptions, UTextureRenderTarget2D* InRenderTarget)
			: FCaptureSource(InMediaCapture, MoveTemp(InCaptureOptions))
			, RenderTarget(InRenderTarget)
		{
		}
		
		virtual EMediaCaptureSourceType GetSourceType() override
        {
        	return EMediaCaptureSourceType::RENDER_TARGET;
        }

		virtual void ResizeSourceBuffer(FIntPoint Size) override
		{
			if (RenderTarget.IsValid())
			{
				RenderTarget->ResizeTarget(Size.X, Size.Y);
			}
		}
		
		virtual bool ValidateSource(const FIntPoint& InDesiredSize, const EPixelFormat InDesiredPixelFormat, const bool bInCurrentlyCapturing) override
		{
			return ValidateTextureRenderTarget2D(RenderTarget.Get(), CaptureOptions, InDesiredSize, InDesiredPixelFormat, bInCurrentlyCapturing);
		}

		virtual bool PostInitialize() override
		{
			return MediaCapture->PostInitializeCaptureRenderTarget(RenderTarget.Get());
		}

		virtual FIntPoint GetSize() override
		{
			if (RenderTarget.IsValid())
			{
				return FIntPoint(RenderTarget->SizeX, RenderTarget->SizeY);
			}

			return FIntPoint();
		}
		
		virtual bool UpdateSourceImpl() override
		{
			if (MediaCapture.IsValid())
			{
				return MediaCapture->UpdateRenderTargetImpl(RenderTarget.Get());
			}
			return false;
		}

		virtual FTextureRHIRef GetSourceTextureForInput_RenderThread()
		{
			constexpr bool bEvenIfPendingKill = false;
			constexpr bool bThreadSafeTest = true;
			if (RenderTarget.IsValid(bEvenIfPendingKill, bThreadSafeTest) && RenderTarget.GetEvenIfUnreachable()->GetRenderTargetResource())
			{
				return RenderTarget.GetEvenIfUnreachable()->GetRenderTargetResource()->GetTextureRenderTarget2DResource()->GetTextureRHI();
			}
			return nullptr;
		}

		virtual UTextureRenderTarget2D* GetRenderTarget() const override
		{
			return RenderTarget.Get();
		}

		virtual bool IsValid() const override
		{
			return RenderTarget.IsValid();
		}

		TWeakObjectPtr<UTextureRenderTarget2D> RenderTarget;
	};

	namespace MediaTextureCapture
	{
		class FMediaTextureTransformPS : public FGlobalShader
		{
		public:
			DECLARE_GLOBAL_SHADER(FMediaTextureTransformPS);
			SHADER_USE_PARAMETER_STRUCT(FMediaTextureTransformPS, FGlobalShader);

			BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
				SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Input)
				SHADER_PARAMETER_STRUCT(FScreenPassTextureViewportParameters, Output)
				SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InputTexture) 
				SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
				SHADER_PARAMETER(FVector4f, RotationScale)
				RENDER_TARGET_BINDING_SLOTS()
			END_SHADER_PARAMETER_STRUCT()

			static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
			{
				return true;
			}

			/** Allocates and setup shader parameter in the incoming graph builder */
			static FParameters* AllocateAndSetParameters(FRDGBuilder& GraphBuilder, const FScreenPassTexture& InputTexture, const FScreenPassTexture& OutputTexture, const FMatrix2x2f& RotationScale)
			{
				FParameters* Parameters = GraphBuilder.AllocParameters<FParameters>();

				FVector4f RotScaleVector;
				RotationScale.GetMatrix(RotScaleVector.X, RotScaleVector.Y, RotScaleVector.Z, RotScaleVector.W);
				
				Parameters->Input = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(InputTexture));
				Parameters->Output = GetScreenPassTextureViewportParameters(FScreenPassTextureViewport(OutputTexture));
				Parameters->InputTexture = InputTexture.Texture;
				Parameters->InputSampler = TStaticSamplerState<SF_Bilinear, AM_Border, AM_Border, AM_Border>::GetRHI();
				Parameters->RotationScale = RotScaleVector;
				Parameters->RenderTargets[0] = FRenderTargetBinding { OutputTexture.Texture, ERenderTargetLoadAction::ENoAction };

				return Parameters;
			}
		};
	
		IMPLEMENT_GLOBAL_SHADER(FMediaTextureTransformPS, "/MediaIOShaders/MediaIO.usf", "MediaTransform", SF_Pixel);
	}
	
	class FMediaTextureCaptureSource : public FCaptureSource
	{
	public:
		FMediaTextureCaptureSource(UMediaCapture* InMediaCapture, const FMediaCaptureOptions& InCaptureOptions, UMediaTexture* InMediaTexture, float InRotation, const FVector2f& InScale)
			: FCaptureSource(InMediaCapture, InCaptureOptions)
			, MediaTexture(InMediaTexture)
		{
			float SinAngle, CosAngle;
			FMath::SinCos(&SinAngle, &CosAngle, FMath::DegreesToRadians(InRotation));

			RotationScale = FMatrix2x2f(
				InScale.X * CosAngle, -InScale.X * SinAngle,
				InScale.Y * SinAngle, InScale.Y * CosAngle);
		}

		virtual bool ValidateSource(const FIntPoint& DesiredSize, const EPixelFormat DesiredPixelFormat, const bool bCurrentlyCapturing) override
		{
			return ValidateMediaTexture(MediaTexture.Get(), CaptureOptions, GetSize(), DesiredSize, DesiredPixelFormat, bCurrentlyCapturing);
		}
		
		virtual bool PostInitialize() override
		{
			if (TStrongObjectPtr<UMediaCapture> PinnedMediaCapture = MediaCapture.Pin())
			{
				return PinnedMediaCapture->PostInitializeCaptureMediaTexture(MediaTexture.Get());
			}
			return false;
		}
		
		virtual EMediaCaptureSourceType GetSourceType() override
		{
			return EMediaCaptureSourceType::MEDIA_TEXTURE;
		}
		
		virtual FIntPoint GetSize() override
		{
			if (TStrongObjectPtr<UMediaTexture> PinnedMediaTexture = MediaTexture.Pin())
			{
				const FVector2f InitialSize = FVector2f(PinnedMediaTexture->GetWidth(), PinnedMediaTexture->GetHeight());
				const FVector2f TransformedSize = RotationScale.TransformVector(InitialSize).GetAbs();
				return FIntPoint(FMath::CeilToInt(TransformedSize.X), FMath::CeilToInt(TransformedSize.Y));
			}

			return FIntPoint();
		}
		
		virtual bool IsValid() const override
		{
			return MediaTexture.IsValid();
		}

		virtual bool UpdateSourceImpl() override
		{
			if (TStrongObjectPtr<UMediaCapture> PinnedMediaCapture = MediaCapture.Pin())
			{
				return PinnedMediaCapture->UpdateMediaTextureImpl(MediaTexture.Get());
			}
			return false;
		}

		virtual FTextureRHIRef GetSourceTextureForInput_RenderThread() override
		{
			constexpr bool bEvenIfPendingKill = false;
			constexpr bool bThreadSafeTest = true;
			if (MediaTexture.IsValid(bEvenIfPendingKill, bThreadSafeTest) && MediaTexture.GetEvenIfUnreachable()->GetResource())
			{
				return MediaTexture.GetEvenIfUnreachable()->GetResource()->GetTextureRHI();
			}
			
			return nullptr;
		}
		
		virtual void AddPasses(FRenderPipeline& Pipeline) override
		{
			// This rendering code, in addition to applying rotation/scale transforms, does an aspect ratio fit to avoid stretching the media texture, and thus should run in all cases,
			// even when the rotation/scale transform is identity. This is acknowledged to be different behavior to other capture methods that will stretch the output image to fit into
			// the output resolution instead.
			FRenderPass TransformRenderPass;
			TransformRenderPass.Name = "TransformMediaTexture";
			TransformRenderPass.OutputType = ERDGViewableResourceType::Texture;
			TransformRenderPass.InitializePassOutputDelegate = FRenderPass::FInitializePassOutput::CreateLambda([](const UE::MediaCapture::FInitializePassOutputArgs& Args, uint32 FrameId)
			{
				check(Args.InputResource && Args.InputResource->Type == ERDGViewableResourceType::Texture);
				FRDGTextureDesc InputResourceDesc = static_cast<FRDGTextureRef>(Args.InputResource)->Desc;

				FRDGTextureDesc TransformOutputTextureDesc = InputResourceDesc;
				TransformOutputTextureDesc.Reset();
				TransformOutputTextureDesc.Flags = TexCreate_None;
				TransformOutputTextureDesc.Flags |= TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_UAV;
				TransformOutputTextureDesc.ClearValue = FClearValueBinding(FLinearColor(0, 0, 0, 0));
				TransformOutputTextureDesc.Extent = Args.MediaCapture->GetDesiredSize();

				TRefCountPtr<IPooledRenderTarget> RenderTarget = AllocatePooledTexture(TransformOutputTextureDesc, TEXT("MediaCapture Transformed Output"));
				return RenderTargetResource(MoveTemp(RenderTarget));
			});
			TransformRenderPass.ExecutePassDelegate = FRenderPass::FExecutePass::CreateLambda([RotScale = RotationScale, Size = GetSize()](const UE::MediaCaptureData::FCaptureFrameArgs& Args, const TSharedPtr<UE::MediaCaptureData::FCaptureFrame>& CapturingFrame, FRDGViewableResource* InputResource, FRDGViewableResource* OutputResource)
			{
				check(InputResource->Type == ERDGViewableResourceType::Texture);

				const float FitScale = FMath::Min(FMath::Min(Args.DesiredSize.X / (float)Size.X, Args.DesiredSize.Y / (float)Size.Y), 1.0f);
				const FMatrix2x2f FitRotScale = RotScale.Concatenate(FMatrix2x2f(1.0f / FitScale));
				const FIntPoint FitSize = FIntPoint(FMath::CeilToInt(Size.X * FitScale), FMath::CeilToInt(Size.Y * FitScale));

				FScreenPassTexture InputTexture(static_cast<FRDGTextureRef>(InputResource));
				FScreenPassRenderTarget Input;
				const FIntVector InputSize = InputTexture.Texture->Desc.GetSize();
				Input.Texture = InputTexture.Texture;
				Input.ViewRect = FIntRect{ 0, 0, InputSize.X, InputSize.Y };
				Input.LoadAction = ERenderTargetLoadAction::ENoAction;
				Input.UpdateVisualizeTextureExtent();

				FScreenPassRenderTarget OutputTexture(static_cast<FRDGTextureRef>(OutputResource), ERenderTargetLoadAction::ENoAction);
				const FIntVector OutputSize = OutputTexture.Texture->Desc.GetSize();
				OutputTexture.ViewRect = FIntRect {
					FMath::CeilToInt(0.5f * (OutputSize.X - FitSize.X)), FMath::CeilToInt(0.5f * (OutputSize.Y - FitSize.Y)),
					FMath::FloorToInt(0.5f * (OutputSize.X + FitSize.X)), FMath::FloorToInt(0.5f * (OutputSize.Y + FitSize.Y))};
				OutputTexture.UpdateVisualizeTextureExtent();

				MediaTextureCapture::FMediaTextureTransformPS::FParameters* PassParameters =
					MediaTextureCapture::FMediaTextureTransformPS::AllocateAndSetParameters(Args.GraphBuilder, InputTexture, OutputTexture, FitRotScale);

				const FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
				const TShaderMapRef<MediaTextureCapture::FMediaTextureTransformPS> PixelShader(GlobalShaderMap);
				FPixelShaderUtils::AddFullscreenPass(
					Args.GraphBuilder,
					GlobalShaderMap,
					RDG_EVENT_NAME("MediaCapture Texture Transform"),
					PixelShader,
					PassParameters,
					OutputTexture.ViewRect);
			});

			Pipeline.AddPass(TransformRenderPass, TEXT("Uncompression"));
		}
		
	private:
		TWeakObjectPtr<UMediaTexture> MediaTexture;
		FMatrix2x2f RotationScale;
	};
}

