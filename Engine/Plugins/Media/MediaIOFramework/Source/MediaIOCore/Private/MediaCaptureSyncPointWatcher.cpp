// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaCaptureSyncPointWatcher.h"

#include "HAL/RunnableThread.h"
#include "MediaCaptureHelper.h"


namespace UE::MediaCaptureData
{

FSyncPointWatcher::FSyncPointWatcher(UMediaCapture* InMediaCapture)
	: MediaCapture(InMediaCapture)
{
	bIsEnabled = true;

	const FString ThreadName = FString::Printf(TEXT("%s_SyncThread"), *MediaCapture->GetName());
	WorkingThread.Reset(FRunnableThread::Create(this, *ThreadName, 256 * 1024, TPri_Highest));
}

uint32 FSyncPointWatcher::Run()
{
	while (bIsEnabled)
	{
		if (PendingCaptureFrames.IsEmpty())
		{
			PendingTaskSignal->Wait();
			if (!bIsEnabled)
			{
				break;
			}
		}

		FPendingCaptureData NextCaptureData;
		{
			if(!PendingCaptureFrames.Dequeue(NextCaptureData))
			{
				continue;
			}
		}

		ProcessPendingCapture(NextCaptureData);
	}

	return 0;
}

void FSyncPointWatcher::ProcessPendingCapture(const FPendingCaptureData& Data)
{
	ON_SCOPE_EXIT
	{
		--PendingCaptureFramesCount;
	};

	double WaitTime = 0.0;
	bool bWaitedForCompletion = false;
	{
		FScopedDurationTimer Timer(WaitTime);

		if (Data.SyncHandler)
		{
			// Wait until fence has been written (shader has completed)
			while (true)
			{
				if (Data.SyncHandler->RHIFence->Poll())
				{
					bWaitedForCompletion = true;
					break;
				}

				if (!Data.CapturingFrame->bMediaCaptureActive)
				{
					bWaitedForCompletion = false;
					break;
				}

				constexpr float SleepTimeSeconds = 50 * 1E-6f;
				FPlatformProcess::SleepNoStats(SleepTimeSeconds);
			}

			Data.SyncHandler->RHIFence->Clear();
			Data.SyncHandler->bIsBusy = false;
		}
		else
		{
			// This is a GPUDirect copy, so use ITextureTransfer's wait method to synchronize with the GPU.
			if (!Data.CapturingFrame->bMediaCaptureActive)
			{
				bWaitedForCompletion = false;
			}
			else if (MediaCapture)
			{
				MediaCapture->WaitForGPU(Data.CapturingFrame->GetTextureResource());
				bWaitedForCompletion = true;
			}
		}
	}

	if (Data.CapturingFrame->bMediaCaptureActive && bWaitedForCompletion && MediaCapture)
	{
		// Ensure that we do not run the following code out of order with respect to the other sibling async tasks,
		// because the Pending Frames are expected to be processed in order.

		if (MediaCapture->UseAnyThreadCapture())
		{
			FMediaCaptureHelper::OnReadbackComplete(MediaCapture, Data.CapturingFrame);
		}
		else
		{
			++MediaCapture->WaitingForRenderCommandExecutionCounter;

			ENQUEUE_RENDER_COMMAND(MediaOutputCaptureReadbackComplete)([OwnerCapture = MediaCapture, CapturingFrame = Data.CapturingFrame](FRHICommandListImmediate&)
			{
				FMediaCaptureHelper::OnReadbackComplete(OwnerCapture, CapturingFrame);
			});
		}
	}
}

void FSyncPointWatcher::QueuePendingCapture(FPendingCaptureData NewData)
{
	++PendingCaptureFramesCount;
	PendingCaptureFrames.Enqueue(MoveTemp(NewData));
	PendingTaskSignal->Trigger();
}

void FSyncPointWatcher::Stop()
{
	bIsEnabled = false;
	PendingTaskSignal->Trigger();
}

void FSyncPointWatcher::WaitForAllPendingTasksToComplete()
{
	// Clean up completed tasks.
	while (PendingCaptureFramesCount > 0)
	{
		FPlatformProcess::SleepNoStats(50e-6f);
	}
}

void FSyncPointWatcher::WaitForSinglePendingTaskToComplete()
{
	const uint32 PendingSnapshot = PendingCaptureFramesCount;
	if (PendingSnapshot > 0)
	{
		// Wait for at least one frame to be completed
		while (PendingCaptureFramesCount.load() >= PendingSnapshot)
		{
			FPlatformProcess::SleepNoStats(50e-6f);
		}
	}
}

FSyncPointWatcher::~FSyncPointWatcher()
{
	constexpr bool bWaitForThread = true;
	WorkingThread->Kill(bWaitForThread);
	WorkingThread.Reset();
}

}
