// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Runnable.h"

#include "HAL/ThreadManager.h"
#include "MediaCapture.h"

class FRunnableThread;

namespace UE::MediaCaptureData
{
	struct FPendingCaptureData
	{
		/** Sync handler holding the fence used for this frame */
		TSharedPtr<UMediaCapture::FMediaCaptureSyncData> SyncHandler;

		/** Frame we are waiting on to be captured */
		TSharedPtr<class FCaptureFrame> CapturingFrame;
	};

	/** 
	 * Class dealing with sync points. Goes over each pending capture frame data
	 * and wait until it's done capturing. Notifies implementation from render thread or watcher's
	 * own thread if supported.
	 */
	class FSyncPointWatcher : public FRunnable
	{
	public:
		FSyncPointWatcher(UMediaCapture* InMediaCapture);
		virtual ~FSyncPointWatcher();

		//~ Begin FRunnable interface
		virtual uint32 Run() override;
		virtual void Stop() override;
		//~ End FRunnable interface

		/** Wait for all pending tasks to be completed */
		void WaitForAllPendingTasksToComplete();

		/** Waits for one pending task to be completed in order to have one free capture frame */
		void WaitForSinglePendingTaskToComplete();

		/** Queue work to be processed by our thread */
		void QueuePendingCapture(FPendingCaptureData NewData);

	private:

		/** Process pending frame being captured */
		void ProcessPendingCapture(const FPendingCaptureData& Data);
		
	protected:

		/** Thread running sync point watch */
		TUniquePtr<FRunnableThread> WorkingThread;

		/** Whether watcher is enabled or not */
		std::atomic<bool> bIsEnabled = false;

		/** Number of pending frames to process */
		std::atomic<uint32> PendingCaptureFramesCount = 0;

		/** Event used to wake up / wait for new work */
		FEventRef PendingTaskSignal;

		/** Media capture linked to this watcher */
		UMediaCapture* MediaCapture = nullptr;

		/** Queue of work */
		TSpscQueue<FPendingCaptureData> PendingCaptureFrames;
	};
}
