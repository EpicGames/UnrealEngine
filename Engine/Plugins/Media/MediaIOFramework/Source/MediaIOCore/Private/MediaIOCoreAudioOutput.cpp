// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaIOCoreAudioOutput.h"

#include "AudioDevice.h"
#include "AudioMixerDevice.h"
#include "AudioMixerSubmix.h"
#include "Engine/Engine.h"
#include "Sound/AudioSettings.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogMediaIOAudioOutput, Log, All);

namespace MediaIOCoreAudioOutputUtils
{
	UWorld* GetCurrentWorld()
	{
#if WITH_EDITOR
		if (GEditor)
		{
			FWorldContext* PIEWorldContext = GEditor->GetPIEWorldContext();
			if (PIEWorldContext)
			{
				return PIEWorldContext->World();
			}
			return GEditor->GetEditorWorldContext(false).World();
		}
		if (GWorld)
		{
			return GWorld->GetWorld();
		}
#endif
		for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
		{
			if (WorldContext.WorldType == EWorldType::Game)
			{
				return WorldContext.World();
			}
		}
		return nullptr;
	}
}

#define LOCTEXT_NAMESPACE "MediaIOAudioOutput"

FMediaIOAudioOutput::FMediaIOAudioOutput(Audio::FPatchOutputStrongPtr InPatchOutput, const FAudioOptions& InAudioOptions)
    : NumInputChannels(InAudioOptions.InNumInputChannels)
    , NumOutputChannels(InAudioOptions.InNumOutputChannels)
    , TargetFrameRate(InAudioOptions.InTargetFrameRate)
    , MaxSampleLatency(InAudioOptions.InMaxSampleLatency)
    , OutputSampleRate(InAudioOptions.InOutputSampleRate)
	, PatchOutput(MoveTemp(InPatchOutput))
{
	NumSamplesPerFrame = FMath::CeilToInt(NumInputChannels * OutputSampleRate / TargetFrameRate.AsDecimal());
}

int32 FMediaIOAudioOutput::GetAudioBuffer(int32 InNumSamplesToPop, float* OutBuffer) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAudioOutput::GetAudioBuffer);

	if (PatchOutput)
	{
		constexpr bool bUseLatestAudio = false;
		return PatchOutput->MixInAudio(OutBuffer, InNumSamplesToPop, bUseLatestAudio);
	}
	return 0;
}

Audio::FAlignedFloatBuffer FMediaIOAudioOutput::GetFloatBuffer(uint32 NumSamplesToGet) const
{
	// NumSamplesToPop must be a multiple of 4 in order to avoid an assertion in the audio pipeline.
	const int32 NumSamplesToPop = Align(NumSamplesToGet, 4);

	Audio::FAlignedFloatBuffer FloatBuffer;
	FloatBuffer.SetNumZeroed(NumSamplesToPop);

	const int32 NumPopped = GetAudioBuffer(NumSamplesToPop, FloatBuffer.GetData());
	FloatBuffer.SetNum(NumPopped);

	return FloatBuffer;
}

/**
 * Implementation of ISubmixBufferListener.
 * 
 * It is necessary to make it an aggregate so it remains valid as SharedRef from within ~FMediaIOAudioCapture().
 * This is because UnregisterSubmixBufferListener requires a valid SharedRef, which AsShared() can't provide
 * from within the destructor.
 */ 
class FMediaIOAudioCapture::FSubmixBufferListenerProxy : public ISubmixBufferListener
{
public:
	FSubmixBufferListenerProxy(const TSharedRef<ISubmixBufferListener>& InSubmixBufferListener)
		: SubmixBufferListenerWeak(InSubmixBufferListener)
	{
	}

	//~ Begin ISubmixBufferListener
	virtual void OnNewSubmixBuffer(const USoundSubmix* InOwningSubmix, float* InAudioData, int32 InNumSamples, int32 InNumChannels, const int32 InSampleRate, double InAudioClock) override
	{
		if (const TSharedPtr<ISubmixBufferListener> SubmixBufferListener = SubmixBufferListenerWeak.Pin())
		{
			SubmixBufferListener->OnNewSubmixBuffer(InOwningSubmix, InAudioData, InNumSamples, InNumChannels, InSampleRate, InAudioClock);
		}
	}

	virtual const FString& GetListenerName() const override
	{
		if (const TSharedPtr<ISubmixBufferListener> SubmixBufferListener = SubmixBufferListenerWeak.Pin())
		{
			return SubmixBufferListener->GetListenerName();
		}

		static const FString ListenerName = TEXT("MediaIO AudioCapture Listener Bridge");
		return ListenerName;
	}
	//~ End ISubmixBufferListener

private:
	TWeakPtr<ISubmixBufferListener> SubmixBufferListenerWeak;
};

FMediaIOAudioCapture::~FMediaIOAudioCapture()
{
	UnregisterAudioDevice();
}

void FMediaIOAudioCapture::Initialize(const FAudioDeviceHandle& InAudioDeviceHandle)
{
	RegisterAudioDevice(InAudioDeviceHandle);
}

void FMediaIOAudioCapture::Initialize(const FAudioDeviceHandle& InAudioDeviceHandle, const FOnAudioCaptured& InAudioCapturedDelegate)
{
	// Important: Assign the delegate before registering the buffer listener to avoid thread safety issues.
	AudioCapturedDelegate = InAudioCapturedDelegate;

	RegisterAudioDevice(InAudioDeviceHandle);
}

void FMediaIOAudioCapture::OnNewSubmixBuffer(const USoundSubmix* InOwningSubmix, float* InAudioData, int32 InNumSamples, int32 InNumChannels, const int32 InSampleRate, double InAudioClock)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMediaIOAudioCapture::OnNewSubmixBuffer);

	check(InOwningSubmix);
	if (InOwningSubmix->GetFName() == PrimarySubmixName)
	{
		if (ensureMsgf(NumChannels == InNumChannels, TEXT("Expected %d channels from submix buffer but got %d instead."), NumChannels, InNumChannels))
		{
			AudioCapturedDelegate.ExecuteIfBound(InAudioData, InNumSamples);
			const int32 NumPushed = AudioSplitter.PushAudio(InAudioData, InNumSamples);
			if (InNumSamples != NumPushed && NumPushed != -1)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FMediaIOAudioCapture::OnNewSubmixBuffer::BufferOverrun);
				UE_LOGF(LogMediaIOAudioOutput, Verbose, "Pushed samples mismatch, Incoming samples: %d, Pushed samples: %d", InNumSamples, NumPushed);
			}
		}
	}
}

TSharedPtr<FMediaIOAudioOutput> FMediaIOAudioCapture::CreateAudioOutput(int32 InNumOutputChannels, FFrameRate InTargetFrameRate, uint32 InMaxSampleLatency, uint32 InOutputSampleRate)
{
	if (NumChannels > InNumOutputChannels)
	{
		UE_LOGF(LogMediaIOAudioOutput, Error, "Audio capture initialization error, please change the audio output channel count to a number greater or equal to %d.", NumChannels);

#if WITH_EDITOR
		const FText WarningText = FText::Format(LOCTEXT("AudioCaptureError", "Audio capture initialization error, please change the audio output channel count to a number greather or equal to %d."), NumChannels);
		FNotificationInfo WarningNotification(WarningText);
		WarningNotification.bFireAndForget = true;
		WarningNotification.ExpireDuration = 6.0f;
		WarningNotification.bUseThrobber = false;

		// For adding notifications.
		FSlateNotificationManager::Get().AddNotification(WarningNotification);
#endif
		return nullptr;
	}
	
	if (ensureMsgf(InOutputSampleRate == SampleRate, TEXT("The engine's sample rate is different from the output sample rate and resampling is not yet supported in Media Captutre.")))
	{
		constexpr float Gain = 1.0f;
		check(InNumOutputChannels > 0);
		
		Audio::FPatchOutputStrongPtr PatchOutput = AudioSplitter.AddNewPatch(InMaxSampleLatency, Gain);
		FMediaIOAudioOutput::FAudioOptions Options;

		Options.InNumInputChannels = NumChannels;
		Options.InNumOutputChannels = InNumOutputChannels;
		Options.InTargetFrameRate = InTargetFrameRate;
		Options.InMaxSampleLatency = InMaxSampleLatency;
		Options.InOutputSampleRate = InOutputSampleRate;

		return MakeShared<FMediaIOAudioOutput>(MoveTemp(PatchOutput), Options);
	}

	return nullptr;
}

void FMediaIOAudioCapture::RegisterBufferListener(FAudioDevice* AudioDevice)
{
	if (AudioDevice)
	{
		const Audio::FMixerDevice* MixerDevice = static_cast<Audio::FMixerDevice*>(AudioDevice);
		NumChannels = MixerDevice->GetDeviceOutputChannels();
		SampleRate = MixerDevice->GetSampleRate();
		PrimarySubmixName = *GetDefault<UAudioSettings>()->MasterSubmix.GetAssetName();

		if (!SubmixBufferListenerProxy)
		{
			SubmixBufferListenerProxy = MakeShared<FSubmixBufferListenerProxy>(AsShared());
		}
		AudioDevice->RegisterSubmixBufferListener(SubmixBufferListenerProxy.ToSharedRef(), AudioDevice->GetMainSubmixObject());
	}
}

const FString& FMediaIOAudioCapture::GetListenerName() const
{
	static const FString ListenerName = TEXT("MediaIO AudioCapture Listener");
	return ListenerName;
}

void FMediaIOAudioCapture::UnregisterBufferListener(FAudioDevice* AudioDevice)
{
	if (AudioDevice && SubmixBufferListenerProxy)
	{
		AudioDevice->UnregisterSubmixBufferListener(SubmixBufferListenerProxy.ToSharedRef(), AudioDevice->GetMainSubmixObject());
	}
}

void FMediaIOAudioCapture::RegisterAudioDevice(const FAudioDeviceHandle& InAudioDeviceHandle)
{
	// Can only be registered to one device.
	UnregisterAudioDevice();

	if (InAudioDeviceHandle.IsValid())
	{
		RegisterBufferListener(InAudioDeviceHandle.GetAudioDevice());
		RegisteredDeviceId = InAudioDeviceHandle.GetDeviceID();
	}
}

void FMediaIOAudioCapture::UnregisterAudioDevice()
{
	if (RegisteredDeviceId != INDEX_NONE && GEngine && GEngine->GetAudioDeviceManager())
	{
		if (FAudioDevice* RegisteredDevice = GEngine->GetAudioDeviceManager()->GetAudioDeviceRaw(RegisteredDeviceId))
		{
			UnregisterBufferListener(RegisteredDevice);
		}
		RegisteredDeviceId = INDEX_NONE;
	}
}

void FMediaIOAudioCapture::SetAudioCapturedDelegate(const FOnAudioCaptured& InAudioCapturedDelegate)
{
	// We need to protect against thread unsafe operation. The audio capture delegate can't be
	// changed if the SubmixBufferListener is registered to an audio device.
	if (ensure(!SubmixBufferListenerProxy || SubmixBufferListenerProxy.GetSharedReferenceCount() == 1))
	{
		AudioCapturedDelegate = InAudioCapturedDelegate;
	}
}

FMainMediaIOAudioCapture::FMainMediaIOAudioCapture()
{	
#if WITH_EDITOR
	FEditorDelegates::PostPIEStarted.AddRaw(this, &FMainMediaIOAudioCapture::OnPIEStarted);
	FEditorDelegates::PrePIEEnded.AddRaw(this, &FMainMediaIOAudioCapture::OnPIEEnded);
#endif
}

FMainMediaIOAudioCapture::~FMainMediaIOAudioCapture()
{
#if WITH_EDITOR
	FEditorDelegates::PrePIEEnded.RemoveAll(this);
	FEditorDelegates::PostPIEStarted.RemoveAll(this);
#endif
}

void FMainMediaIOAudioCapture::Initialize()
{
	// Register the current device (PIE or main).
	RegisterCurrentAudioDevice();
}

void FMainMediaIOAudioCapture::Initialize(const FOnAudioCaptured& InAudioCapturedDelegate)
{
	// Important: Assign the delegate before registering the buffer listener to avoid thread safety issues.
	SetAudioCapturedDelegate(InAudioCapturedDelegate);

	// Register the current device (PIE or main).
	RegisterCurrentAudioDevice();
}

#if WITH_EDITOR
void FMainMediaIOAudioCapture::OnPIEStarted(const bool)
{
	RegisterCurrentAudioDevice();
}

void FMainMediaIOAudioCapture::OnPIEEnded(const bool)
{
	// Note: PIE context still active, need to explicitly fallback to engine's device.
	RegisterMainAudioDevice();
}
#endif

void FMainMediaIOAudioCapture::RegisterMainAudioDevice()
{
	RegisterAudioDevice(GEngine->GetMainAudioDevice());
}

void FMainMediaIOAudioCapture::RegisterCurrentAudioDevice()
{
	if (const UWorld* World = MediaIOCoreAudioOutputUtils::GetCurrentWorld())
	{
		RegisterAudioDevice(World->GetAudioDevice());
	}
	else
	{
		RegisterMainAudioDevice();
	}
}

#undef LOCTEXT_NAMESPACE //MediaIOAudioOutput