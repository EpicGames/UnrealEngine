// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaIOCoreCommonDisplayMode.h"

#define LOCTEXT_NAMESPACE "CommonDisplayMode"

namespace MediaIOCommonDisplayModeInfo
{
	const FMediaIOCommonDisplayModeResolutionInfo AllCommonDisplayModeResolutions[] = {
		FMediaIOCommonDisplayModeResolutionInfo{ 720,	486,	LOCTEXT("ModeRes_NTSC",		"NTSC SD")	},
		FMediaIOCommonDisplayModeResolutionInfo{ 720,	576,	LOCTEXT("ModeRes_PAL",		"PAL SD")	},
		FMediaIOCommonDisplayModeResolutionInfo{ 1280,	720,	LOCTEXT("ModeRes_720",		"HD 720")	},
		FMediaIOCommonDisplayModeResolutionInfo{ 1920,	1080,	LOCTEXT("ModeRes_1080",		"HD 1080")	},
		FMediaIOCommonDisplayModeResolutionInfo{ 2048,	1080,	LOCTEXT("ModeRes_2DCI",		"2K DCI")	},
		FMediaIOCommonDisplayModeResolutionInfo{ 2048,	1556,	LOCTEXT("ModeRes_2Full",	"2K Full")	},
		FMediaIOCommonDisplayModeResolutionInfo{ 3840,	2160,	LOCTEXT("ModeRes_4UHD",		"4K UHD")	},
		FMediaIOCommonDisplayModeResolutionInfo{ 4096,	2160,	LOCTEXT("ModeRes_4DCI",		"4K DCI")	},
		FMediaIOCommonDisplayModeResolutionInfo{ 7680,	4320,	LOCTEXT("ModeRes_8UDH",		"8K UHD")	},
		FMediaIOCommonDisplayModeResolutionInfo{ 8192,	4320,	LOCTEXT("ModeRes_8DCI",		"8K DCI")	},
		FMediaIOCommonDisplayModeResolutionInfo{ 15360,	8640,	LOCTEXT("ModeRes_16DCI",	"16K DCI")	},
		FMediaIOCommonDisplayModeResolutionInfo{ 61440,	34560,	LOCTEXT("ModeRes_64DCI",	"64K DCI")	},
		};

	const FMediaIOCommonDisplayModeInfo AllCommonDisplayModes[] = {
		FMediaIOCommonDisplayModeInfo{ 720,		486,	FFrameRate(30000, 1001),	EMediaIOStandardType::Interlaced,					LOCTEXT("Mode_NTSC", "NTSC") },
		FMediaIOCommonDisplayModeInfo{ 720,		486,	FFrameRate(60000, 1001),	EMediaIOStandardType::Progressive,					LOCTEXT("Mode_NTSCp", "NTSCp") },
		FMediaIOCommonDisplayModeInfo{ 720,		576,	FFrameRate(25, 1),			EMediaIOStandardType::Interlaced,					LOCTEXT("Mode_PAL", "PAL") },
		FMediaIOCommonDisplayModeInfo{ 720,		576,	FFrameRate(50, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_PALp", "PALp") },

		FMediaIOCommonDisplayModeInfo{ 1280,	720,	FFrameRate(50, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_720p50", "720p50") },
		FMediaIOCommonDisplayModeInfo{ 1280,	720,	FFrameRate(60000, 1001),	EMediaIOStandardType::Progressive,					LOCTEXT("Mode_720p5994", "720p5994") },
		FMediaIOCommonDisplayModeInfo{ 1280,	720,	FFrameRate(60, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_720p60", "720p60") },

		FMediaIOCommonDisplayModeInfo{ 1920,	1080,	FFrameRate(24000, 1001),	EMediaIOStandardType::Progressive,					LOCTEXT("Mode_1080p2398", "1080p2398") },
		FMediaIOCommonDisplayModeInfo{ 1920,	1080,	FFrameRate(24000, 1001),	EMediaIOStandardType::ProgressiveSegmentedFrame,	LOCTEXT("Mode_1080sf2398", "1080sf2398") },
		FMediaIOCommonDisplayModeInfo{ 1920,	1080,	FFrameRate(24, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_1080p24", "1080p24") },
		FMediaIOCommonDisplayModeInfo{ 1920,	1080,	FFrameRate(24, 1),			EMediaIOStandardType::ProgressiveSegmentedFrame,	LOCTEXT("Mode_1080sf24", "1080sf24") },
		FMediaIOCommonDisplayModeInfo{ 1920,	1080,	FFrameRate(25, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_1080p25", "1080p25") },
		FMediaIOCommonDisplayModeInfo{ 1920,	1080,	FFrameRate(25, 1),			EMediaIOStandardType::ProgressiveSegmentedFrame,	LOCTEXT("Mode_1080sf25", "1080sf25") },
		FMediaIOCommonDisplayModeInfo{ 1920,	1080,	FFrameRate(30000, 1001),	EMediaIOStandardType::Progressive,					LOCTEXT("Mode_1080p2997", "1080p2997") },
		FMediaIOCommonDisplayModeInfo{ 1920,	1080,	FFrameRate(30000, 1001),	EMediaIOStandardType::ProgressiveSegmentedFrame,	LOCTEXT("Mode_1080sf2997", "1080sf2997") },
		FMediaIOCommonDisplayModeInfo{ 1920,	1080,	FFrameRate(30, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_1080p30", "1080p30") },
		FMediaIOCommonDisplayModeInfo{ 1920,	1080,	FFrameRate(30, 1),			EMediaIOStandardType::ProgressiveSegmentedFrame,	LOCTEXT("Mode_1080sf30", "1080sf30") },
		FMediaIOCommonDisplayModeInfo{ 1920,	1080,	FFrameRate(25, 1),			EMediaIOStandardType::Interlaced,					LOCTEXT("Mode_1080i50", "1080i50") },
		FMediaIOCommonDisplayModeInfo{ 1920,	1080,	FFrameRate(30000, 1001),	EMediaIOStandardType::Interlaced,					LOCTEXT("Mode_1080i5994", "1080i5994") },
		FMediaIOCommonDisplayModeInfo{ 1920,	1080,	FFrameRate(30, 1),			EMediaIOStandardType::Interlaced,					LOCTEXT("Mode_1080i60", "1080i60") },
		FMediaIOCommonDisplayModeInfo{ 1920,	1080,	FFrameRate(50, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_1080p50", "1080p50") },
		FMediaIOCommonDisplayModeInfo{ 1920,	1080,	FFrameRate(60000, 1001),	EMediaIOStandardType::Progressive,					LOCTEXT("Mode_1080p5994", "1080p5994") },
		FMediaIOCommonDisplayModeInfo{ 1920,	1080,	FFrameRate(60, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_1080p60", "1080p60") },

		FMediaIOCommonDisplayModeInfo{ 2048,	1080,	FFrameRate(24000, 1001),	EMediaIOStandardType::Progressive,					LOCTEXT("Mode_2kDCI2398", "2kDCI2398") },
		FMediaIOCommonDisplayModeInfo{ 2048,	1080,	FFrameRate(24, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_2kDCI24", "2kDCI24") },
		FMediaIOCommonDisplayModeInfo{ 2048,	1080,	FFrameRate(25, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_2kDCI25", "2kDCI25") },
		FMediaIOCommonDisplayModeInfo{ 2048,	1080,	FFrameRate(30000, 1001),	EMediaIOStandardType::Progressive,					LOCTEXT("Mode_2kDCI2997", "2kDCI2997") },
		FMediaIOCommonDisplayModeInfo{ 2048,	1080,	FFrameRate(30, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_2kDCI30", "2kDCI30") },
		FMediaIOCommonDisplayModeInfo{ 2048,	1080,	FFrameRate(50, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_2kDCI50", "2kDCI50") },
		FMediaIOCommonDisplayModeInfo{ 2048,	1080,	FFrameRate(60000, 1001),	EMediaIOStandardType::Progressive,					LOCTEXT("Mode_2kDCI5994", "2kDCI5994") },
		FMediaIOCommonDisplayModeInfo{ 2048,	1080,	FFrameRate(60, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_2kDCI60", "2kDCI60") },

		FMediaIOCommonDisplayModeInfo{ 2048,	1556,	FFrameRate(24000, 1001),	EMediaIOStandardType::Progressive,					LOCTEXT("Mode_2k2398", "2k2398") },
		FMediaIOCommonDisplayModeInfo{ 2048,	1556,	FFrameRate(24, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_2k24", "2k24") },
		FMediaIOCommonDisplayModeInfo{ 2048,	1556,	FFrameRate(25, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_2k25", "2k25") },

		FMediaIOCommonDisplayModeInfo{ 3840,	2160,	FFrameRate(24000, 1001),	EMediaIOStandardType::Progressive,					LOCTEXT("Mode_4kUHD2398", "4kUHD2398") },
		FMediaIOCommonDisplayModeInfo{ 3840,	2160,	FFrameRate(24, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_4kUHD24", "4kUHD24") },
		FMediaIOCommonDisplayModeInfo{ 3840,	2160,	FFrameRate(25, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_4kUHD25", "4kUHD25") },
		FMediaIOCommonDisplayModeInfo{ 3840,	2160,	FFrameRate(30000, 1001),	EMediaIOStandardType::Progressive,					LOCTEXT("Mode_4kUHD2997", "4kUHD2997") },
		FMediaIOCommonDisplayModeInfo{ 3840,	2160,	FFrameRate(30, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_4kUHD30", "4kUHD30") },
		FMediaIOCommonDisplayModeInfo{ 3840,	2160,	FFrameRate(50, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_4kUHD50", "4kUHD50") },
		FMediaIOCommonDisplayModeInfo{ 3840,	2160,	FFrameRate(60000, 1001),	EMediaIOStandardType::Progressive,					LOCTEXT("Mode_4kUHD5994", "4kUHD5994") },
		FMediaIOCommonDisplayModeInfo{ 3840,	2160,	FFrameRate(60, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_4kUHD60", "4kUHD60") },

		FMediaIOCommonDisplayModeInfo{ 4096,	2160,	FFrameRate(24000, 1001),	EMediaIOStandardType::Progressive,					LOCTEXT("Mode_4kDCI2398", "4kDCI2398") },
		FMediaIOCommonDisplayModeInfo{ 4096,	2160,	FFrameRate(24, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_4kDCI24", "4kDCI24") },
		FMediaIOCommonDisplayModeInfo{ 4096,	2160,	FFrameRate(25, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_4kDCI25", "4kDCI25") },
		FMediaIOCommonDisplayModeInfo{ 4096,	2160,	FFrameRate(30000, 1001),	EMediaIOStandardType::Progressive,					LOCTEXT("Mode_4kDCI2997", "4kDCI2997") },
		FMediaIOCommonDisplayModeInfo{ 4096,	2160,	FFrameRate(30, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_4kDCI30", "4kDCI30") },
		FMediaIOCommonDisplayModeInfo{ 4096,	2160,	FFrameRate(50, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_4kDCI50", "4kDCI50") },
		FMediaIOCommonDisplayModeInfo{ 4096,	2160,	FFrameRate(60000, 1001),	EMediaIOStandardType::Progressive,					LOCTEXT("Mode_4kDCI5994", "4kDCI5994") },
		FMediaIOCommonDisplayModeInfo{ 4096,	2160,	FFrameRate(60, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_4kDCI60", "4kDCI60") },

		FMediaIOCommonDisplayModeInfo{ 7680,	4320,	FFrameRate(24000, 1001),	EMediaIOStandardType::Progressive,					LOCTEXT("Mode_8kUHD2398", "8kUHD2398") },
		FMediaIOCommonDisplayModeInfo{ 7680,	4320,	FFrameRate(24, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_8kUHD24", "8kUHD24") },
		FMediaIOCommonDisplayModeInfo{ 7680,	4320,	FFrameRate(25, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_8kUHD25", "8kUHD25") },
		FMediaIOCommonDisplayModeInfo{ 7680,	4320,	FFrameRate(30000, 1001),	EMediaIOStandardType::Progressive,					LOCTEXT("Mode_8kUHD2997", "8kUHD2997") },
		FMediaIOCommonDisplayModeInfo{ 7680,	4320,	FFrameRate(30, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_8kUHD30", "8kUHD30") },
		FMediaIOCommonDisplayModeInfo{ 7680,	4320,	FFrameRate(50, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_8kUHD50", "8kUHD50") },
		FMediaIOCommonDisplayModeInfo{ 7680,	4320,	FFrameRate(60000, 1001),	EMediaIOStandardType::Progressive,					LOCTEXT("Mode_8kUHD5994", "8kUHD5994") },
		FMediaIOCommonDisplayModeInfo{ 7680,	4320,	FFrameRate(60, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_8kUHD60", "8kUHD60") },

		FMediaIOCommonDisplayModeInfo{ 8192,	4320,	FFrameRate(24000, 1001),	EMediaIOStandardType::Progressive,					LOCTEXT("Mode_8kDCI2398", "8kDCI2398") },
		FMediaIOCommonDisplayModeInfo{ 8192,	4320,	FFrameRate(24, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_8kDCI24", "8kDCI24") },
		FMediaIOCommonDisplayModeInfo{ 8192,	4320,	FFrameRate(25, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_8kDCI25", "8kDCI25") },
		FMediaIOCommonDisplayModeInfo{ 8192,	4320,	FFrameRate(30000, 1001),	EMediaIOStandardType::Progressive,					LOCTEXT("Mode_8kDCI2997", "8kDCI2997") },
		FMediaIOCommonDisplayModeInfo{ 8192,	4320,	FFrameRate(30, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_8kDCI30", "8kDCI30") },
		FMediaIOCommonDisplayModeInfo{ 8192,	4320,	FFrameRate(50, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_8kDCI50", "8kDCI50") },
		FMediaIOCommonDisplayModeInfo{ 8192,	4320,	FFrameRate(60000, 1001),	EMediaIOStandardType::Progressive,					LOCTEXT("Mode_8kDCI5994", "8kDCI5994") },
		FMediaIOCommonDisplayModeInfo{ 8192,	4320,	FFrameRate(60, 1),			EMediaIOStandardType::Progressive,					LOCTEXT("Mode_8kDCI60", "8kDCI60") },
	};
}

TArrayView<const FMediaIOCommonDisplayModeInfo> FMediaIOCommonDisplayModes::GetAllModes()
{
	return MediaIOCommonDisplayModeInfo::AllCommonDisplayModes;
}

TArrayView<const FMediaIOCommonDisplayModeResolutionInfo> FMediaIOCommonDisplayModes::GetAllResolutions()
{
	return MediaIOCommonDisplayModeInfo::AllCommonDisplayModeResolutions;
}

const FMediaIOCommonDisplayModeResolutionInfo* FMediaIOCommonDisplayModes::Find(int32 InWidth, int32 InHeight)
{
	const int32 NumModeResolutions = UE_ARRAY_COUNT(MediaIOCommonDisplayModeInfo::AllCommonDisplayModeResolutions);
	for(int32 Index = 0; Index < NumModeResolutions; ++Index)
	{
		const FMediaIOCommonDisplayModeResolutionInfo* Info = MediaIOCommonDisplayModeInfo::AllCommonDisplayModeResolutions + Index;
		if (Info->Width == InWidth && Info->Height == InHeight)
		{
			return Info;
		}
	}
	return nullptr;
}

const FMediaIOCommonDisplayModeInfo* FMediaIOCommonDisplayModes::Find(int32 InWidth, int32 InHeight, const FFrameRate& InFrameRate, EMediaIOStandardType InStandard)
{
	const int32 NumModes = UE_ARRAY_COUNT(MediaIOCommonDisplayModeInfo::AllCommonDisplayModes);
	for (int32 Index = 0; Index < NumModes; ++Index)
	{
		const FMediaIOCommonDisplayModeInfo* Info = MediaIOCommonDisplayModeInfo::AllCommonDisplayModes + Index;
		if (Info->Width == InWidth && Info->Height == InHeight && FMath::IsNearlyEqual(Info->FrameRate.AsDecimal(), InFrameRate.AsDecimal()) && Info->Standard == InStandard)
		{
			return Info;
		}
	}
	return nullptr;
}

FText FMediaIOCommonDisplayModes::GetMediaIOCommonDisplayModeResolutionInfoName(int32 InWidth, int32 InHeight)
{
	const FMediaIOCommonDisplayModeResolutionInfo* Resolution = Find(InWidth, InHeight);
	if (Resolution)
	{
		return Resolution->Name;
	}
	return FText::Format(LOCTEXT("Undefined_Resolution", "{0}x{1}"), FText::AsNumber(InWidth), FText::AsNumber(InHeight));
}

FText FMediaIOCommonDisplayModes::GetMediaIOCommonDisplayModeInfoName(int32 InWidth, int32 InHeight, const FFrameRate& InFrameRate, EMediaIOStandardType InStandard)
{
	const FMediaIOCommonDisplayModeInfo* Mode = Find(InWidth, InHeight, InFrameRate, InStandard);
	if(Mode)
	{
		return Mode->Name;
	}

	FNumberFormattingOptions FpsFormattingOptions;
	FpsFormattingOptions.SetAlwaysSign(false)
		.SetUseGrouping(false)
		.SetMaximumFractionalDigits(2);

	FText Standard = FText::GetEmpty();
	switch(InStandard)
	{
	case EMediaIOStandardType::Interlaced: Standard = LOCTEXT("Interlaced_Short", "i"); break;
	case EMediaIOStandardType::Progressive: Standard = LOCTEXT("Progressive_Short", "p"); break;
	case EMediaIOStandardType::ProgressiveSegmentedFrame: Standard = LOCTEXT("ProgressiveSegmentedFrame_Short", "psf"); break;
	}

	const FMediaIOCommonDisplayModeResolutionInfo* Resolution = Find(InWidth, InHeight);
	if (Resolution)
	{
		return FText::Format(LOCTEXT("Undefined_Mode", "{0}x{1}{2}"), Resolution->Name, FText::AsNumber(InFrameRate.AsDecimal(), &FpsFormattingOptions), Standard);
	}

	return FText::Format(LOCTEXT("Undefined_ModeResolution", "{0}x{1}x{2}{3}"), FText::AsNumber(InWidth), FText::AsNumber(InHeight), FText::AsNumber(InFrameRate.AsDecimal(), &FpsFormattingOptions), Standard);
}

#undef LOCTEXT_NAMESPACE
