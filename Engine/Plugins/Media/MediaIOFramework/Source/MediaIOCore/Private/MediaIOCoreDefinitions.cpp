// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaIOCoreDefinitions.h"
#include "IMediaIOCoreDeviceProvider.h"
#include "Containers/ArrayView.h"
#include "MediaIOCoreCommonDisplayMode.h"
#include "Misc/FrameRate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaIOCoreDefinitions)

#define LOCTEXT_NAMESPACE "MediaIOCoreDefinitions"

/**
 * MediaIOCoreDefinitions
 */
namespace MediaIOCoreDefinitions
{
	const int32 InvalidDeviceIdentifier = -1;
	const int32 InvalidDevicePortIdentifier = -1;
	const int32 InvalidDeviceModeIdentifier = -1;
	
	const FName NAME_Protocol = TEXT("protocol");
	
	const TCHAR* DeviceStr = TEXT("device");
	const TCHAR* SingleStr = TEXT("single");
	const TCHAR* DualStr = TEXT("dual");
	const TCHAR* QuadSquareStr = TEXT("quadSQ");
	const TCHAR* QuadTsiStr = TEXT("quadSI");
	const TCHAR* HDMIStr = TEXT("HDMI");
	
	const TCHAR* GetTransportString(EMediaIOTransportType InLinkType, EMediaIOQuadLinkTransportType InQuadLinkType)
	{
		const TCHAR* Result = MediaIOCoreDefinitions::SingleStr;
		switch (InLinkType)
		{
		case EMediaIOTransportType::DualLink:
			Result = MediaIOCoreDefinitions::DualStr;
			break;
		case EMediaIOTransportType::HDMI:
			Result = MediaIOCoreDefinitions::HDMIStr;
			break;
		case EMediaIOTransportType::QuadLink:
		{
			Result = MediaIOCoreDefinitions::QuadSquareStr;
			if (InQuadLinkType == EMediaIOQuadLinkTransportType::TwoSampleInterleave)
			{
				Result = MediaIOCoreDefinitions::QuadTsiStr;
			}
			break;
		}
		}
		return Result;
	}
}


/**
 * FMediaIOCoreDevice
 */
FMediaIODevice::FMediaIODevice()
	: DeviceIdentifier(MediaIOCoreDefinitions::InvalidDeviceIdentifier)
{}


bool FMediaIODevice::operator==(const FMediaIODevice& Other) const
{
	return Other.DeviceIdentifier == DeviceIdentifier;
}


bool FMediaIODevice::IsValid() const
{
	return DeviceIdentifier != MediaIOCoreDefinitions::InvalidDeviceIdentifier;
}


/**
 * FMediaIOConnection
 */
FMediaIOConnection::FMediaIOConnection()
	: Protocol(MediaIOCoreDefinitions::NAME_Protocol)
	, TransportType(EMediaIOTransportType::SingleLink)
	, QuadTransportType(EMediaIOQuadLinkTransportType::SquareDivision)
	, PortIdentifier(MediaIOCoreDefinitions::InvalidDevicePortIdentifier)
{}


bool FMediaIOConnection::operator==(const FMediaIOConnection& Other) const
{
	return Other.Device == Device
		&& Other.TransportType == TransportType
		&& Other.PortIdentifier == PortIdentifier
		&& (Other.TransportType != EMediaIOTransportType::QuadLink || Other.QuadTransportType == QuadTransportType);
}


FString FMediaIOConnection::ToUrl() const
{
	if (IsValid())
	{
		return FString::Printf(TEXT("%s://%s%d/%s%d")
			, *Protocol.ToString()
			, MediaIOCoreDefinitions::DeviceStr
			, Device.DeviceIdentifier
			, MediaIOCoreDefinitions::GetTransportString(TransportType, QuadTransportType)
			, PortIdentifier
			);
	}
	return Protocol.ToString();
}


bool FMediaIOConnection::IsValid() const
{
	return Device.IsValid() && PortIdentifier != MediaIOCoreDefinitions::InvalidDevicePortIdentifier;
}

/**
 * FMediaIOMode
 */
FMediaIOMode::FMediaIOMode()
	: FrameRate(30, 1)
	, Resolution(1920, 1080)
	, Standard(EMediaIOStandardType::Progressive)
	, DeviceModeIdentifier(MediaIOCoreDefinitions::InvalidDeviceModeIdentifier)
{}


bool FMediaIOMode::operator==(const FMediaIOMode& Other) const
{
	return Other.DeviceModeIdentifier == DeviceModeIdentifier;
}


FText FMediaIOMode::GetModeName() const
{
	if (IsValid())
	{
		FFrameRate FieldFrameRate = FrameRate;
		if (Standard == EMediaIOStandardType::Interlaced)
		{
			FieldFrameRate.Numerator /= 2;
		}
		return FMediaIOCommonDisplayModes::GetMediaIOCommonDisplayModeInfoName(Resolution.X, Resolution.Y, FieldFrameRate, Standard);
	}
	return LOCTEXT("InvalidMode", "<Invalid>");
}


bool FMediaIOMode::IsValid() const
{
	return DeviceModeIdentifier != MediaIOCoreDefinitions::InvalidDeviceModeIdentifier;
}


/**
 * FMediaIOConfiguration
 */
FMediaIOConfiguration::FMediaIOConfiguration()
	: bIsInput(true)
{}


bool FMediaIOConfiguration::operator== (const FMediaIOConfiguration& Other) const
{
	return MediaConnection == Other.MediaConnection
		&& MediaMode == Other.MediaMode
		&& bIsInput == Other.bIsInput;
}


bool FMediaIOConfiguration::IsValid() const
{
	return MediaConnection.IsValid() && MediaMode.IsValid();
}


/**
 * FMediaIOInputConfiguration
 */
FMediaIOInputConfiguration::FMediaIOInputConfiguration()
	: InputType(EMediaIOInputType::Fill)
	, KeyPortIdentifier(MediaIOCoreDefinitions::InvalidDevicePortIdentifier)
{
	MediaConfiguration.bIsInput = true;
}


bool FMediaIOInputConfiguration::operator== (const FMediaIOInputConfiguration& Other) const
{
	if (InputType == Other.InputType && MediaConfiguration == Other.MediaConfiguration)
	{
		if (InputType == EMediaIOInputType::FillAndKey)
		{
			if (KeyPortIdentifier != Other.KeyPortIdentifier)
			{
				return false;
			}
		}
		return true;
	}
	return false;
}

bool FMediaIOInputConfiguration::IsValid() const
{
	bool bResult = false;
	if (MediaConfiguration.IsValid())
	{
		bResult = true;
		if (InputType == EMediaIOInputType::FillAndKey)
		{
			if (KeyPortIdentifier == MediaIOCoreDefinitions::InvalidDevicePortIdentifier)
			{
				bResult = false;
			}
		}
	}
	return bResult;
}

/**
 * FMediaIOOutputConfiguration
 */
FMediaIOOutputConfiguration::FMediaIOOutputConfiguration()
	: OutputType(EMediaIOOutputType::Fill)
	, KeyPortIdentifier(MediaIOCoreDefinitions::InvalidDevicePortIdentifier)
	, OutputReference(EMediaIOReferenceType::FreeRun)
	, ReferencePortIdentifier(MediaIOCoreDefinitions::InvalidDevicePortIdentifier)
{
	MediaConfiguration.bIsInput = false;
}


bool FMediaIOOutputConfiguration::operator== (const FMediaIOOutputConfiguration& Other) const
{
	if (OutputType == Other.OutputType
		&& MediaConfiguration == Other.MediaConfiguration
		&& OutputReference == Other.OutputReference)
	{
		if (OutputType == EMediaIOOutputType::FillAndKey)
		{
			if (KeyPortIdentifier != Other.KeyPortIdentifier)
			{
				return false;
			}
		}
		if (OutputReference == EMediaIOReferenceType::Input)
		{
			if (ReferencePortIdentifier != Other.ReferencePortIdentifier)
			{
				return false;
			}
		}
		return true;
	}
	return false;
}


bool FMediaIOOutputConfiguration::IsValid() const
{
	bool bResult = false;
	if (MediaConfiguration.IsValid())
	{
		bResult = true;
		if (OutputType == EMediaIOOutputType::FillAndKey)
		{
			if (KeyPortIdentifier == MediaIOCoreDefinitions::InvalidDevicePortIdentifier)
			{
				bResult = false;
			}
		}
		if (OutputReference == EMediaIOReferenceType::Input)
		{
			if (ReferencePortIdentifier == MediaIOCoreDefinitions::InvalidDevicePortIdentifier)
			{
				bResult = false;
			}
		}
	}
	return bResult;
}

bool FMediaIOVideoTimecodeConfiguration::IsValid() const
{
	return TimecodeFormat != EMediaIOAutoDetectableTimecodeFormat::None && MediaConfiguration.IsValid();
}

bool FMediaIOVideoTimecodeConfiguration::operator== (const FMediaIOVideoTimecodeConfiguration& Other) const
{
	return MediaConfiguration == Other.MediaConfiguration
		&& TimecodeFormat == Other.TimecodeFormat;
}

FText FMediaIOVideoTimecodeConfiguration::ToText(bool bAutoDetected) const
{
	if (bAutoDetected)
	{
		const FText ConfigurationText = FText::Format(LOCTEXT("FMediaIOAutoDetectConfigurationToText", "{0} [device{1}/{2}{3}]")
        			, FText::FromName(MediaConfiguration.MediaConnection.Device.DeviceName)
        			, FText::AsNumber(MediaConfiguration.MediaConnection.Device.DeviceIdentifier)
        			, IMediaIOCoreDeviceProvider::GetTransportName(MediaConfiguration.MediaConnection.TransportType, MediaConfiguration.MediaConnection.QuadTransportType)
        			, FText::AsNumber(MediaConfiguration.MediaConnection.PortIdentifier)
        		);
		
		return FText::Format(LOCTEXT("MediaTimecodeAutoDetectConfigurationRefText", "{0} / Auto"), ConfigurationText);
	}
	
	if (IsValid())
	{
		FText Timecode = LOCTEXT("InvalidTimecode", "<Invalid>");
		switch(TimecodeFormat)
		{
		case EMediaIOAutoDetectableTimecodeFormat::LTC:
			Timecode = LOCTEXT("LTCLabel", "LTC");
			break;
		case EMediaIOAutoDetectableTimecodeFormat::VITC:
			Timecode = LOCTEXT("VITCLabel", "VITC");
			break;
		}

		FText ConfigurationText = FText::Format(LOCTEXT("FMediaIOConfigurationToText", "{0} - {1} [device{2}/{3}{4}/{5}]")
			, MediaConfiguration.bIsInput ? LOCTEXT("In", "In") : LOCTEXT("Out", "Out")
			, FText::FromName(MediaConfiguration.MediaConnection.Device.DeviceName)
			, FText::AsNumber(MediaConfiguration.MediaConnection.Device.DeviceIdentifier)
			, IMediaIOCoreDeviceProvider::GetTransportName(MediaConfiguration.MediaConnection.TransportType, MediaConfiguration.MediaConnection.QuadTransportType)
			, FText::AsNumber(MediaConfiguration.MediaConnection.PortIdentifier)
			, MediaConfiguration.MediaMode.GetModeName()
		);

		return FText::Format(LOCTEXT("MediaTimecodeConfigurationRefText", "{0}/{1}")
			, ConfigurationText
			, Timecode
		);
	}
	return LOCTEXT("InvalidConfiguration", "<Invalid>");
}


#undef LOCTEXT_NAMESPACE

