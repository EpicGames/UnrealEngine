// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaIOCoreDeinterlacer.h"

#include "Async/ParallelFor.h"
#include "ColorManagement/ColorManagementDefines.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaIOCoreDeinterlacer)

namespace UE::MediaIOCore
{
	static TAutoConsoleVariable<bool> CVarDeinterlacerStaggerFrames(
		TEXT("MediaIO.Deinterlacer.StaggerFrames"),
		true,
		TEXT("When true, the second field sample produced by the bob deinterlacer is timestamped one field period after the first so both samples render. When false, both samples share the source frame time (legacy behavior; only one is rendered)."),
		ECVF_Default);

	TArray<TSharedRef<FMediaIOCoreTextureSampleBase>> FDeinterlacer::Deinterlace(const FVideoFrame& InVideoFrame) const
 	{
		TArray<TSharedRef<FMediaIOCoreTextureSampleBase>> Samples;

		const TSharedPtr<FMediaIOCoreTextureSampleBase> TextureSample = AcquireSampleDelegate.Execute();

		if (TextureSample && TextureSample->Initialize(InVideoFrame.VideoBuffer
			, InVideoFrame.BufferSize
			, InVideoFrame.Stride
			, InVideoFrame.Width
			, InVideoFrame.Height
			, InVideoFrame.SampleFormat
			, InVideoFrame.Time
			, InVideoFrame.FrameRate
			, InVideoFrame.Timecode
			, InVideoFrame.SourceColorSettings))
		{
			Samples.Add(TextureSample.ToSharedRef());
		}

		return Samples;
 	}

	TArray<TSharedRef<FMediaIOCoreTextureSampleBase>> FBobDeinterlacer::Deinterlace(const FVideoFrame& InVideoFrame) const
	{
		TArray<TSharedRef<FMediaIOCoreTextureSampleBase>> Samples;

		const TSharedPtr<FMediaIOCoreTextureSampleBase> TextureSampleEven = AcquireSampleDelegate.Execute();
		const TSharedPtr<FMediaIOCoreTextureSampleBase> TextureSampleOdd = AcquireSampleDelegate.Execute();

		if (TextureSampleEven && TextureSampleOdd)
		{
			TArray<uint8> EvenBuffer;
			EvenBuffer.Reserve(InVideoFrame.BufferSize);

			TArray<uint8> OddBuffer;
			OddBuffer.Reserve(InVideoFrame.BufferSize);

			for (uint32 IndexY = (InterlaceFieldOrder == EMediaIOInterlaceFieldOrder::TopFieldFirst ? 0 : 1); IndexY < InVideoFrame.Height; IndexY += 2)
			{
				const uint8* Source = reinterpret_cast<const uint8*>(InVideoFrame.VideoBuffer) + (IndexY * InVideoFrame.Stride);
				EvenBuffer.Append(Source, InVideoFrame.Stride);
				EvenBuffer.Append(Source, InVideoFrame.Stride);
			}

			for (uint32 IndexY = (InterlaceFieldOrder == EMediaIOInterlaceFieldOrder::TopFieldFirst ? 1 : 0); IndexY < InVideoFrame.Height; IndexY += 2)
			{
				const uint8* Source = reinterpret_cast<const uint8*>(InVideoFrame.VideoBuffer) + (IndexY * InVideoFrame.Stride);
				OddBuffer.Append(Source, InVideoFrame.Stride);
				OddBuffer.Append(Source, InVideoFrame.Stride);
			}

			// Bob produces two output samples per source frame, representing the temporal moments
			// when each field was captured. They must be timestamped one FrameRate-period apart so
			// the renderer treats them as non-overlapping in time; otherwise both samples cover the
			// same interval and only one is ever rendered. AJA passes the field rate as FrameRate
			// for interlaced sources (see AjaDeviceProvider.cpp), so AsInterval() is one field period.
			// Gated by cvar so the legacy (shared-timestamp) behavior can be restored at runtime.
			const FTimespan FieldDuration = CVarDeinterlacerStaggerFrames.GetValueOnAnyThread()
				? FTimespan(ETimespan::TicksPerSecond * InVideoFrame.FrameRate.AsInterval())
				: FTimespan::Zero();

			if (TextureSampleEven->Initialize(
				  MoveTemp(EvenBuffer)
				, InVideoFrame.Stride
				, InVideoFrame.Width
				, InVideoFrame.Height
				, InVideoFrame.SampleFormat
				, InVideoFrame.Time
				, InVideoFrame.FrameRate
				, InVideoFrame.Timecode
				, InVideoFrame.SourceColorSettings))
			{
				Samples.Add(TextureSampleEven.ToSharedRef());
			}

			// Don't create second sample if first one fails in order to avoid introducing field flipping.
			if (Samples.Num() && TextureSampleOdd->Initialize(
				  MoveTemp(OddBuffer)
				, InVideoFrame.Stride
				, InVideoFrame.Width
				, InVideoFrame.Height
				, InVideoFrame.SampleFormat
				, InVideoFrame.Time + FieldDuration
				, InVideoFrame.FrameRate
				, InVideoFrame.Timecode
				, InVideoFrame.SourceColorSettings))
			{
				Samples.Add(TextureSampleOdd.ToSharedRef());
			}
		}

		return Samples;
	}

	TArray<TSharedRef<FMediaIOCoreTextureSampleBase>> FBlendDeinterlacer::Deinterlace(const FVideoFrame& InVideoFrame) const
	{
		TArray<TSharedRef<FMediaIOCoreTextureSampleBase>> Samples;

		if (const TSharedPtr<FMediaIOCoreTextureSampleBase> TextureSample = AcquireSampleDelegate.Execute())
		{
			TArray<uint8> OutputBuffer;
			OutputBuffer.AddZeroed(InVideoFrame.BufferSize);

			const uint8* Source = static_cast<const uint8*>(InVideoFrame.VideoBuffer);
			const uint32 Stride = InVideoFrame.Stride;
			const uint32 Height = InVideoFrame.Height;
			// TopFieldFirst: top-field source lines are at even Y. BottomFieldFirst: at odd Y.
			const uint32 TopFieldParity = (InterlaceFieldOrder == EMediaIOInterlaceFieldOrder::TopFieldFirst) ? 0 : 1;

			// Average each top-field line with its adjacent bottom-field line, then emit the same
			// averaged row twice to fill the full output height. Equivalent to "line-double both
			// fields then per-pixel average" but in one pass.
			ParallelFor(Height / 2, [&](int32 PairIndex)
			{
				const uint32 TopY = static_cast<uint32>(PairIndex) * 2 + TopFieldParity;
				const uint32 BotY = TopY ^ 1;

				const uint8* TopLine = Source + (TopY * Stride);
				const uint8* BotLine = Source + (BotY * Stride);

				const uint32 OutY0 = static_cast<uint32>(PairIndex) * 2;
				uint8* DestA = OutputBuffer.GetData() + (OutY0 * Stride);
				uint8* DestB = OutputBuffer.GetData() + ((OutY0 + 1) * Stride);

				for (uint32 X = 0; X < Stride; X++)
				{
					const uint8 Avg = static_cast<uint8>(FMath::DivideAndRoundUp(TopLine[X] + BotLine[X], 2));
					DestA[X] = Avg;
					DestB[X] = Avg;
				}
			});

			if (TextureSample->Initialize(
				  MoveTemp(OutputBuffer)
				, InVideoFrame.Stride
				, InVideoFrame.Width
				, InVideoFrame.Height
				, InVideoFrame.SampleFormat
				, InVideoFrame.Time
				, InVideoFrame.FrameRate
				, InVideoFrame.Timecode
				, InVideoFrame.SourceColorSettings))
			{
				Samples.Add(TextureSample.ToSharedRef());
			}
		}

		return Samples;
	}

	TArray<TSharedRef<FMediaIOCoreTextureSampleBase>> FDiscardDeinterlacer::Deinterlace(const FVideoFrame& InVideoFrame) const
	{
		TArray<TSharedRef<FMediaIOCoreTextureSampleBase>> Samples;
		if (const TSharedPtr<FMediaIOCoreTextureSampleBase> TextureSample = AcquireSampleDelegate.Execute())
		{
			TArray<uint8> Buffer;
			Buffer.Reserve(InVideoFrame.BufferSize);

			for (uint32 YIndex = InterlaceFieldOrder == EMediaIOInterlaceFieldOrder::TopFieldFirst ? 0 : 1; YIndex < InVideoFrame.Height; YIndex += 2)
			{
				const uint8* Source = reinterpret_cast<const uint8*>(InVideoFrame.VideoBuffer) + (YIndex * InVideoFrame.Stride);
				Buffer.Append(Source, InVideoFrame.Stride);
				Buffer.Append(Source, InVideoFrame.Stride);
			}

			if (TextureSample->Initialize(
				  MoveTemp(Buffer)
				, InVideoFrame.Stride
				, InVideoFrame.Width
				, InVideoFrame.Height
				, InVideoFrame.SampleFormat
				, InVideoFrame.Time
				, InVideoFrame.FrameRate
				, InVideoFrame.Timecode
				, InVideoFrame.SourceColorSettings))
			{
				Samples.Add(TextureSample.ToSharedRef());
			}
		}

		return Samples;
	}
}
