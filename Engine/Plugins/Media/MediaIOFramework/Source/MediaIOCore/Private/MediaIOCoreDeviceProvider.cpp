// Copyright Epic Games, Inc. All Rights Reserved.

#include "IMediaIOCoreDeviceProvider.h"
#include "MediaIOCoreCommonDisplayMode.h"


#define LOCTEXT_NAMESPACE "MediaIOCoreDeviceProvider"


FText IMediaIOCoreDeviceProvider::GetTransportName(EMediaIOTransportType InLinkType, EMediaIOQuadLinkTransportType InQuadLinkType)
{
	FText Transport = LOCTEXT("Single", "single");
	if (InLinkType == EMediaIOTransportType::DualLink)
	{
		Transport = LOCTEXT("Dual", "dual");
	}
	else if (InLinkType == EMediaIOTransportType::QuadLink && InQuadLinkType == EMediaIOQuadLinkTransportType::SquareDivision)
	{
		Transport = LOCTEXT("QuadSquare", "quadSQ");
	}
	else if (InLinkType == EMediaIOTransportType::QuadLink && InQuadLinkType == EMediaIOQuadLinkTransportType::TwoSampleInterleave)
	{
		Transport = LOCTEXT("QuadTSI", "quadSI");
	}
	else if (InLinkType == EMediaIOTransportType::HDMI)
	{
		Transport = LOCTEXT("HDMI", "HDMI");
	}
	return Transport;
}

FText IMediaIOCoreDeviceProvider::ToText(const FMediaIOConfiguration& InConfiguration, bool bInIsAutoDetected) const
{
	if (bInIsAutoDetected)
	{
		return FText::Format(LOCTEXT("FMediaIOConfigurationToTextAutoDetect", "{0} - {1} [device{2}/{3}{4}/auto]")
				, InConfiguration.bIsInput ? LOCTEXT("In", "In") : LOCTEXT("Out", "Out")
				, FText::FromName(InConfiguration.MediaConnection.Device.DeviceName)
				, FText::AsNumber(InConfiguration.MediaConnection.Device.DeviceIdentifier)
				, GetTransportName(InConfiguration.MediaConnection.TransportType, InConfiguration.MediaConnection.QuadTransportType)
				, FText::AsNumber(InConfiguration.MediaConnection.PortIdentifier)
				);
	}
	if (InConfiguration.IsValid())
	{
		return FText::Format(LOCTEXT("FMediaIOConfigurationToText", "{0} - {1} [device{2}/{3}{4}/{5}]")
			, InConfiguration.bIsInput ? LOCTEXT("In", "In") : LOCTEXT("Out", "Out")
			, FText::FromName(InConfiguration.MediaConnection.Device.DeviceName)
			, FText::AsNumber(InConfiguration.MediaConnection.Device.DeviceIdentifier)
			, GetTransportName(InConfiguration.MediaConnection.TransportType, InConfiguration.MediaConnection.QuadTransportType)
			, FText::AsNumber(InConfiguration.MediaConnection.PortIdentifier)
			, InConfiguration.MediaMode.GetModeName()
			);
	}
	return LOCTEXT("Invalid", "<Invalid>");
}


FText IMediaIOCoreDeviceProvider::ToText(const FMediaIOConnection& InConnection) const
{
	if (InConnection.IsValid())
	{
		return FText::Format(LOCTEXT("FMediaIOConnectionToText", "{0} [device{1}/{2}{3}]")
			, FText::FromName(InConnection.Device.DeviceName)
			, FText::AsNumber(InConnection.Device.DeviceIdentifier)
			, GetTransportName(InConnection.TransportType, InConnection.QuadTransportType)
			, FText::AsNumber(InConnection.PortIdentifier)
			);
	}
	return LOCTEXT("Invalid", "<Invalid>");
}


FText IMediaIOCoreDeviceProvider::ToText(const FMediaIODevice& InDevice) const
{
	if (InDevice.IsValid())
	{
		return FText::Format(LOCTEXT("FMediaIODeviceToText", "{0} [device{1}]")
				, FText::FromName(InDevice.DeviceName)
				, InDevice.DeviceIdentifier
				);
	}
	return LOCTEXT("Invalid", "<Invalid>");
}


FText IMediaIOCoreDeviceProvider::ToText(const FMediaIOMode& InMode) const
{
	return InMode.GetModeName();
}


FText IMediaIOCoreDeviceProvider::ToText(const FMediaIOInputConfiguration& InConfiguration) const
{
	if (InConfiguration.IsValid())
	{
		FText KeyText = FText::GetEmpty();
		if (InConfiguration.InputType == EMediaIOInputType::FillAndKey)
		{
			KeyText = FText::Format(LOCTEXT("FMediaIOInputConfigurationKeyToText", " - Key[{0}{1}]")
				, GetTransportName(InConfiguration.MediaConfiguration.MediaConnection.TransportType, InConfiguration.MediaConfiguration.MediaConnection.QuadTransportType)
				, FText::AsNumber(InConfiguration.KeyPortIdentifier)
			);
		}

		return FText::Format(LOCTEXT("FMediaIOInputConfigurationToText", "Fill - {0} [device{1}/{2}{3}/{4}{5}]")
			, FText::FromName(InConfiguration.MediaConfiguration.MediaConnection.Device.DeviceName)
			, FText::AsNumber(InConfiguration.MediaConfiguration.MediaConnection.Device.DeviceIdentifier)
			, GetTransportName(InConfiguration.MediaConfiguration.MediaConnection.TransportType, InConfiguration.MediaConfiguration.MediaConnection.QuadTransportType)
			, FText::AsNumber(InConfiguration.MediaConfiguration.MediaConnection.PortIdentifier)
			, InConfiguration.MediaConfiguration.MediaMode.GetModeName()
			, KeyText
		);
	}
	return LOCTEXT("Invalid", "<Invalid>");
}


FText IMediaIOCoreDeviceProvider::ToText(const FMediaIOOutputConfiguration& InConfiguration) const
{
	if (InConfiguration.IsValid())
	{
		FText KeyText = FText::GetEmpty();
		if (InConfiguration.OutputType == EMediaIOOutputType::FillAndKey)
		{
			KeyText = FText::Format(LOCTEXT("FMediaIOOutputConfigurationKeyToText", " - Key[{0}{1}]")
				, GetTransportName(InConfiguration.MediaConfiguration.MediaConnection.TransportType, InConfiguration.MediaConfiguration.MediaConnection.QuadTransportType)
				, FText::AsNumber(InConfiguration.KeyPortIdentifier)
				);
		}

		FText ReferenceText = FText::GetEmpty();
		if (InConfiguration.OutputReference == EMediaIOReferenceType::Input)
		{
			ReferenceText = FText::Format(LOCTEXT("FMediaIOOutputConfigurationRefToText", " - Ref[{0}{1}]")
				, GetTransportName(InConfiguration.MediaConfiguration.MediaConnection.TransportType, InConfiguration.MediaConfiguration.MediaConnection.QuadTransportType)
				, FText::AsNumber(InConfiguration.ReferencePortIdentifier)
			);
		}
		else if (InConfiguration.OutputReference == EMediaIOReferenceType::External)
		{
			ReferenceText = LOCTEXT("External", "External");
		}
		else if (InConfiguration.OutputReference == EMediaIOReferenceType::FreeRun)
		{
			ReferenceText = LOCTEXT("FreeRun", "FreeRun");
		}

		return FText::Format(LOCTEXT("FMediaIOOutputConfigurationToText", "Fill - {0} [device{1}/{2}{3}/{4}]{5}{6}")
			, FText::FromName(InConfiguration.MediaConfiguration.MediaConnection.Device.DeviceName)
			, FText::AsNumber(InConfiguration.MediaConfiguration.MediaConnection.Device.DeviceIdentifier)
			, GetTransportName(InConfiguration.MediaConfiguration.MediaConnection.TransportType, InConfiguration.MediaConfiguration.MediaConnection.QuadTransportType)
			, FText::AsNumber(InConfiguration.MediaConfiguration.MediaConnection.PortIdentifier)
			, InConfiguration.MediaConfiguration.MediaMode.GetModeName()
			, KeyText
			, ReferenceText
			);
	}
	return LOCTEXT("Invalid", "<Invalid>");
}

#undef LOCTEXT_NAMESPACE
