// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaIOCoreFileWriter.h"

#include "Containers/UnrealString.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/Paths.h"
#include "Templates/UniquePtr.h"

namespace MediaIOCoreFileWriter
{
	void WriteRawFile(const FString& InFilename, uint8* InBuffer, uint32 InSize, bool bAppend)
	{
#if ALLOW_DEBUG_FILES
		if (!InFilename.IsEmpty())
		{
			FString OutputDirectory = FPaths::Combine(*FPaths::ProjectDir(), TEXT("Saved"), TEXT("Media"));
			FPaths::NormalizeDirectoryName(OutputDirectory);

			if (!FPaths::DirectoryExists(OutputDirectory))
			{
				if (!IFileManager::Get().MakeDirectory(*OutputDirectory, true))
				{
					return;
				}
			}

			FString OutputFilename = FPaths::Combine(OutputDirectory, InFilename);

			// Append current date and time
			FDateTime CurrentDateAndTime = FDateTime::Now();
			if (bAppend)
			{
				OutputFilename += FString(".raw");
			}
			else
			{
				OutputFilename += FString("_") + CurrentDateAndTime.ToString() + FString(".raw");
			}

			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			if (bAppend || !PlatformFile.FileExists(*OutputFilename))
			{
				TUniquePtr<IFileHandle> FileHandle;
				FileHandle.Reset(PlatformFile.OpenWrite(*OutputFilename, bAppend));
				if (FileHandle)
				{
					FileHandle->Write(InBuffer, InSize);
				}
			}
		}
#endif
	}
}
