// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaIOCoreModule.h"

#include "IMediaIOCoreDeviceProvider.h"
#include "IMediaIOCoreModule.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "ShaderCore.h"


DEFINE_LOG_CATEGORY(LogMediaIOCore);

/**
 * Implements the MediaIOCore module.
 */
class FMediaIOCoreModule : public IMediaIOCoreModule
{
public:
	virtual void StartupModule() override
	{
		FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("MediaIOFramework"))->GetBaseDir(), TEXT("Shaders"));
		AddShaderSourceDirectoryMapping(TEXT("/MediaIOShaders"), PluginShaderDir);
	}
	
	virtual void RegisterDeviceProvider(IMediaIOCoreDeviceProvider* InProvider) override
	{
		if (InProvider)
		{
			DeviceProviders.Add(InProvider);
		}
	}

	virtual void UnregisterDeviceProvider(IMediaIOCoreDeviceProvider* InProvider) override
	{
		DeviceProviders.RemoveSingleSwap(InProvider);
	}

	virtual IMediaIOCoreDeviceProvider* GetDeviceProvider(FName InProviderName) override
	{
		for(IMediaIOCoreDeviceProvider* DeviceProvider : DeviceProviders)
		{
			if (DeviceProvider->GetFName() == InProviderName)
			{
				return DeviceProvider;
			}
		}
		return nullptr;
	}

	virtual TConstArrayView<IMediaIOCoreDeviceProvider*> GetDeviceProviders() const override
	{
		return DeviceProviders;
	}

private:
	TArray<IMediaIOCoreDeviceProvider*> DeviceProviders;
};

bool IMediaIOCoreModule::IsAvailable()
{
	return FModuleManager::Get().IsModuleLoaded("MediaIOCore");
}

IMediaIOCoreModule& IMediaIOCoreModule::Get()
{
	return FModuleManager::LoadModuleChecked<FMediaIOCoreModule>("MediaIOCore");
}




IMPLEMENT_MODULE(FMediaIOCoreModule, MediaIOCore);
