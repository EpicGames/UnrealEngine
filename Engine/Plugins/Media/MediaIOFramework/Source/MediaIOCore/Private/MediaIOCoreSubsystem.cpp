// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaIOCoreSubsystem.h"
#include "AudioDeviceManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaIOCoreSubsystem)

void UMediaIOCoreSubsystem::Initialize(FSubsystemCollectionBase& InCollection)
{
	DeviceDestroyedHandle = FAudioDeviceManagerDelegates::OnAudioDeviceDestroyed.AddUObject(this, &UMediaIOCoreSubsystem::OnAudioDeviceDestroyed);
}

void UMediaIOCoreSubsystem::Deinitialize()
{
	FAudioDeviceManagerDelegates::OnAudioDeviceDestroyed.Remove(DeviceDestroyedHandle);
}

TSharedPtr<FMediaIOAudioOutput> UMediaIOCoreSubsystem::CreateAudioOutput(const FCreateAudioOutputArgs& InArgs)
{
	TSharedPtr<FMediaIOAudioCapture> MediaIOAudioCapture;
	using FOnAudioCaptured = FMediaIOAudioCapture::FOnAudioCaptured;

	if (InArgs.AudioDeviceHandle.IsValid())
	{
		if (const TSharedPtr<FMediaIOAudioCapture, ESPMode::ThreadSafe>* FoundMediaIOAudioCapture = MediaIOAudioCaptures.Find(InArgs.AudioDeviceHandle.GetDeviceID()))
		{
			MediaIOAudioCapture = *FoundMediaIOAudioCapture;
		}
		else
		{
			const FOnAudioCaptured OnAudioCapturedDelegate = FOnAudioCaptured::CreateUObject(this, &UMediaIOCoreSubsystem::OnBufferReceivedByCapture, InArgs.AudioDeviceHandle.GetDeviceID());
			MediaIOAudioCapture = MediaIOAudioCaptures.Add(InArgs.AudioDeviceHandle.GetDeviceID(), MakeShared<FMediaIOAudioCapture>());
			MediaIOAudioCapture->Initialize(InArgs.AudioDeviceHandle, OnAudioCapturedDelegate);
		}
	}
	else
	{
		// Fallback using the main audio device.
		if (!MainMediaIOAudioCapture)
		{
			const FOnAudioCaptured OnAudioCapturedDelegate = FOnAudioCaptured::CreateUObject(this, &UMediaIOCoreSubsystem::OnBufferReceivedByCapture, InArgs.AudioDeviceHandle.GetDeviceID());
			MainMediaIOAudioCapture = MakeShared<FMainMediaIOAudioCapture>();
			MainMediaIOAudioCapture->Initialize(OnAudioCapturedDelegate);
		}
		MediaIOAudioCapture = MainMediaIOAudioCapture;
	}	

	return MediaIOAudioCapture->CreateAudioOutput(InArgs.NumOutputChannels, InArgs.TargetFrameRate, InArgs.MaxSampleLatency, InArgs.OutputSampleRate);
}

int32 UMediaIOCoreSubsystem::GetNumAudioInputChannels() const
{
	if (MainMediaIOAudioCapture)
	{
		return MainMediaIOAudioCapture->GetNumInputChannels();
	}
	return 0;
}

void UMediaIOCoreSubsystem::OnAudioDeviceDestroyed(Audio::FDeviceId InAudioDeviceId)
{
	if (MediaIOAudioCaptures.Contains(InAudioDeviceId))
	{
		MediaIOAudioCaptures.Remove(InAudioDeviceId);
	}
}

void UMediaIOCoreSubsystem::OnBufferReceivedByCapture(float* Data, int32 NumSamples, Audio::FDeviceId AudioDeviceID) const
{
	BufferReceivedDelegate.Broadcast(AudioDeviceID, Data, NumSamples);
}
