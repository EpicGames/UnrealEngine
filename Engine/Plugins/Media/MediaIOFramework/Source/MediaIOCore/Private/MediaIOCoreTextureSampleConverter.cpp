// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaIOCoreTextureSampleConverter.h"

#include "MediaIOCorePlayerBase.h"
#include "MediaIOCoreTextureSampleBase.h"


void FMediaIOCoreTextureSampleConverter::Setup(const TSharedPtr<FMediaIOCoreTextureSampleBase>& InSample)
{
	// This proxy sample will be exposed to the media framework
	JITRProxySample = InSample;
}

bool FMediaIOCoreTextureSampleConverter::Convert(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDestinationTexture, const FConversionHints& Hints)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(JITR);

	// Pin the sample instance
	TSharedPtr<FMediaIOCoreTextureSampleBase> SamplePtr = JITRProxySample.Pin();
	if (SamplePtr.IsValid() == false)
	{
		return false;
	}

	// Get owning player to run JITR path
	TSharedPtr<FMediaIOCorePlayerBase> PlayerPtr = SamplePtr->GetPlayer();
	if (PlayerPtr.IsValid() == false)
	{
		return false;
	}

	// Let the player pick a proper sample and render it into this proxy's texture (JITR)
	return PlayerPtr->JustInTimeSampleRender_RenderThread(RHICmdList, InDestinationTexture, SamplePtr);
}

uint32 FMediaIOCoreTextureSampleConverter::GetConverterInfoFlags() const
{
	// JITR requires PreprocessOnly conversion flag as the main idea is to replace
	// the dummy data of the JITR proxy sample with a real data of a sample that
	// will be chosen down the pipe. However, it may return other values when inherited.
	return IMediaTextureSampleConverter::ConverterInfoFlags_PreprocessOnly;
}
