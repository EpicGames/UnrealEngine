// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/SpscQueue.h"
#include "Logging/LogMacros.h"
#include "MediaCapture.h"
#include "MediaCaptureRenderPass.h"
#include "MediaIOCoreModule.h"
#include "RenderGraphUtils.h"
#include "RenderingThread.h"
#include "RHIGPUReadback.h"
#include "ShaderParameterMacros.h"
#include "UObject/ObjectPtr.h"

namespace UE::MediaCaptureData
{

	/** Helper struct to contain arguments for CaptureFrame */
	struct FCaptureFrameArgs
	{
		FRDGBuilder& GraphBuilder;
		TObjectPtr<UMediaCapture> MediaCapture = nullptr;
		FTextureRHIRef ResourceToCapture;
		FRDGTextureRef RDGResourceToCapture = nullptr;
		FIntPoint DesiredSize = FIntPoint::ZeroValue;
		FIntRect SourceViewRect{ 0,0,0,0 };

		EPixelFormat GetFormat() const
		{
			if (RDGResourceToCapture)
			{
				return RDGResourceToCapture->Desc.Format;
			}
			return ResourceToCapture->GetFormat();
		}

		bool HasValidResource() const
		{
			return ResourceToCapture || RDGResourceToCapture;
		}

		FIntPoint GetSizeXY() const
		{
			if (RDGResourceToCapture)
			{
				return FIntPoint(RDGResourceToCapture->Desc.GetSize().X, RDGResourceToCapture->Desc.GetSize().Y);
			}
			return ResourceToCapture->GetSizeXY();
		}

		int32 GetSizeX() const
		{
			return GetSizeXY().X;
		}

		int32 GetSizeY() const
		{
			return GetSizeXY().Y;
		}
	};

	class FFrame
	{
	public:
		virtual ~FFrame() = default;
		/** Get the ID of a frame (Starts at 0 when a capture is started and is incremented on every frame captured). */
		virtual int32 GetId() const = 0;
		/** Returns whether we're doing waiting for a readback to complete or it's in the process of doing a GPU copy. */
		virtual bool IsPending() const = 0;
	};

	/**
	 * Manages a queue of frames used by MediaCapture.
	 */
	class FFrameManager
	{
	public:
		~FFrameManager()
		{
			ENQUEUE_RENDER_COMMAND(MediaCaptureFrameManagerCleaning)(
				[FramesToBeReleased = MoveTemp(Frames)](FRHICommandListImmediate& RHICmdList) mutable
			{
				FramesToBeReleased.Reset();
			});
		}
		
		/** Execute a function for all frames. */
		void ForEachFrame(TFunctionRef<void(const TSharedPtr<FFrame>&)> Function)
		{
			for (const TSharedPtr<FFrame>& Frame : Frames)
			{
				Function(Frame);
			}
		}

		/** Get a string representation of the state of all managed frames (ie. Is frame pending or idle). */
		FString GetFramesState()
		{
			TStringBuilder<256> ReadbackInfoBuilder;
			ReadbackInfoBuilder << "\n";
			for (int32 Index = 0; Index < Frames.Num(); Index++)
			{
				ReadbackInfoBuilder << FString::Format(TEXT("Frame {0} State:  {1}\n"), { Index, Frames[Index]->IsPending() ? TEXT("Pending") : TEXT("Idle") });
			}

			return ReadbackInfoBuilder.ToString();
		}

		/** Returns the list of managed frames. */
		TConstArrayView<TSharedPtr<FFrame>> GetFrames() const
		{
			return Frames;
		}

		/** Add a new frame to the list of frames used by the manager. */
		void AddFrame(TSharedPtr<FFrame> NewFrame)
		{
			AvailableFrames.Enqueue(NewFrame->GetId());
			Frames.Add(MoveTemp(NewFrame));
		}

		/** Mark a frame as available, meaning it can be used for a capture. */
		void MarkAvailable(const FFrame& InFrame)
		{
			if (ensure(!InFrame.IsPending()))
			{
				AvailableFrames.Enqueue(InFrame.GetId());
			}
		}

		/** Mark a frame as pending, meaning it and its resources are being actively used by the media capture. */
		void MarkPending(const FFrame& InFrame)
		{
			if (ensure(InFrame.IsPending()))
			{
				PendingFrames.Enqueue(InFrame.GetId());
			}
		}

		/** Mark a frame as available (*Must* match the next pending frame in the queue). */
		void CompleteNextPending(const FFrame& InFrame)
		{
			TOptional<int32> NextPending = PendingFrames.Dequeue();
			if (ensure(NextPending.IsSet() && InFrame.GetId() == NextPending.GetValue()))
			{
				MarkAvailable(InFrame);
			}
		}

		/** Get the next available (Not in use) frame. */
		template<typename FrameType>
		TSharedPtr<FrameType> GetNextAvailable()
		{
			TSharedPtr<FrameType> NextAvailableFrame;
			
			static_assert(TIsDerivedFrom<FrameType, FFrame>::Value, "The frame type must derive from FFrame.");
			TOptional<int32> NextAvailable = AvailableFrames.Dequeue();
			if (NextAvailable.IsSet())
			{
				NextAvailableFrame = StaticCastSharedPtr<FrameType>(Frames[NextAvailable.GetValue()]);
			}

			// Peek next pending before capturing to avoid getting the newly captured one. 
			// It shouldn't happen but if it happens it will start waiting for the capture to complete
			TSharedPtr<FFrame> NextPendingFrame = PeekNextPending<FFrame>();
			if (NextPendingFrame && NextAvailableFrame)
			{
				ensureMsgf(NextPendingFrame->GetId() != NextAvailableFrame->GetId(), TEXT("The next pending frame was the same as the next available frame, this should not have happened!"));
			}

			return NextAvailableFrame;
		}

		/** Get the next pending frame from the queue without dequeuing it. */
		template<typename FrameType>
		TSharedPtr<FrameType> PeekNextPending()
		{
			static_assert(TIsDerivedFrom<FrameType, FFrame>::Value, "The frame type must derive from FFrame.");
			if (const int32* NextPending = PendingFrames.Peek())
			{
				return StaticCastSharedPtr<FrameType>(Frames[*NextPending]);
			}

			return nullptr;
		}

	private:
		/** List of frames managed by this object. */
		TArray<TSharedPtr<FFrame>> Frames;
		/** List of available frames. */
		TSpscQueue<int32> AvailableFrames;
		/** List of frames in use. */
		TSpscQueue<int32> PendingFrames;
	};

	/** Base implementation of a capture frame. Contains common options and resources shared by texture and buffer frames. */
	class FCaptureFrame : public FFrame
	{
	public:
		FCaptureFrame(int32 InFrameId)
			: FrameId(InFrameId)
		{
		}

		//~ FFrame interface
		virtual int32 GetId() const override
		{
			return FrameId;
		}

		//~ FFrame interface
		virtual bool IsPending() const override
		{
			return bReadbackRequested || bDoingGPUCopy; 
		}
	
		/** Returns true if its output resource is valid */
		virtual bool HasValidResource() const
		{
			return GetTextureResource() || GetBufferResource();	
		}

		/** Simple way to validate the resource type and cast safely */
		virtual bool IsTextureResource() const = 0;
		/** Simple way to validate the resource type and cast safely */
		virtual bool IsBufferResource() const = 0;

		/** Locks the readback resource and returns a pointer to access data from system memory */
		virtual void* Lock(FRHICommandListImmediate& RHICmdList, int32& OutRowStride) = 0;

		/** Unsafe version of the Lock method, used outside of the rendering thread. */
		virtual void* Lock_Unsafe(int32& OutRowStride) { return nullptr; }

		/** Unlocks the readback resource */
		virtual void Unlock() = 0;

		/** Unsafe version of the Unlock method, used outside of the rendering thread. */
		virtual void Unlock_Unsafe() {}

		/** Returns true if the readback is ready to be used */
		virtual bool IsReadbackReady(FRHIGPUMask GPUMask) = 0;
	
		/** Enqueue a copy pass on the rdg builder. */
		virtual void EnqueueCopy(FRDGBuilder& RDGBuilder, FRDGViewableResource* ResourceToReadback, bool bIsAnyThreadSupported) = 0;

		/**
		 * @return The last texture in the capture pipeline that will be readback.
		 * @note This will only return a valid texture after CaptureFrame has been called once since it will initialize the resource.
		 */
		virtual FRHITexture* GetTextureResource() const
		{
			if (IsTextureResource())
			{
				return RenderPassResources.GetLastRenderTarget()->GetRHI();
			}

			return nullptr;
		}

		/**
		 * @return The last buffer in the capture pipeline that will be readback.
		 * @note This will only return a valid buffer after CaptureFrame has been called once since it will initialize the resource.
		 */ 
		virtual FRHIBuffer* GetBufferResource() const
		{
			if (IsBufferResource())
			{
				return RenderPassResources.GetLastBuffer()->GetRHI();
			}

			return nullptr;
		}

	public:
		/** Index that increments for every capture. */
		int32 FrameId = 0;
		/** Frame and timecode data. */
		UMediaCapture::FCaptureBaseData CaptureBaseData;
		/** Whether readback was requested. */
		std::atomic<bool> bReadbackRequested = false;
		/** Whether the captured resource is undergoing a gpy copy at the moment. */
		std::atomic<bool> bDoingGPUCopy = false;
		/** Flag that keeps track of whether MediaCapture is still actively capturing. */
		std::atomic<bool> bMediaCaptureActive = true;
		TSharedPtr<FMediaCaptureUserData> UserData;
		/** Resources used by the media capture render pipeline (ie. Color conversion render target ) */
		UE::MediaCapture::FRenderPassFrameResources RenderPassResources;
		/** Whether this frame is used to preroll the output. Preroll frames are only used to create render resources to avoid a startup hitch. */
		bool bPrerollFrame = true;
	};

	/** Parameter to make our sync pass needing the convert pass as a prereq */
	BEGIN_SHADER_PARAMETER_STRUCT(FMediaCaptureTextureSyncPassParameters, )
		RDG_TEXTURE_ACCESS(Resource, ERHIAccess::CopySrc)
	END_SHADER_PARAMETER_STRUCT()

	BEGIN_SHADER_PARAMETER_STRUCT(FMediaCaptureBufferSyncPassParameters, )
		RDG_BUFFER_ACCESS(Resource, ERHIAccess::CopySrc)
	END_SHADER_PARAMETER_STRUCT()

	/** Capture frame implementation for texture resources. */
	class FTextureCaptureFrame : public FCaptureFrame
	{
	public:
		/** Type alias for the output resource type used during capture frame */
		using FOutputResourceType = FRDGTextureRef;
		using PassParametersType = FMediaCaptureTextureSyncPassParameters;

		FTextureCaptureFrame(int32 InFrameId)
			: FCaptureFrame(InFrameId)
		{
		}


		//~ Begin FCaptureFrame interface
		virtual bool IsTextureResource() const override
		{
			return true;
		}

		virtual bool IsBufferResource() const override
		{
			return false;
		}

		virtual void* Lock(FRHICommandListImmediate& RHICmdList, int32& OutRowStride) override
		{
			if (ReadbackTexture->IsReady() == false)
			{
				UE_LOGF(LogMediaIOCore, Verbose, "Fence for texture readback was not ready");
			}

			int32 ReadbackWidth;
			void* ReadbackPointer = ReadbackTexture->Lock(ReadbackWidth);
			OutRowStride = ReadbackWidth * UMediaCapture::GetBytesPerPixel(GetTextureResource()->GetDesc().Format);
			return ReadbackPointer;
		}

		virtual void* Lock_Unsafe(int32& OutRowStride) override
		{
			void* ReadbackPointer = nullptr;
			int32 ReadbackWidth, ReadbackHeight;
			GDynamicRHI->RHIMapStagingSurface(ReadbackTexture->DestinationStagingTextures[ReadbackTexture->GetLastCopyGPUMask().GetFirstIndex()], nullptr, ReadbackPointer, ReadbackWidth, ReadbackHeight, ReadbackTexture->GetLastCopyGPUMask().GetFirstIndex());
			OutRowStride = ReadbackWidth * UMediaCapture::GetBytesPerPixel(GetTextureResource()->GetDesc().Format);
			return ReadbackPointer;
		}

		virtual void Unlock() override
		{
			ReadbackTexture->Unlock();
		}

		virtual void Unlock_Unsafe() override
		{
			GDynamicRHI->RHIUnmapStagingSurface(ReadbackTexture->DestinationStagingTextures[ReadbackTexture->GetLastCopyGPUMask().GetFirstIndex()], ReadbackTexture->GetLastCopyGPUMask().GetFirstIndex());
		}

		virtual bool IsReadbackReady(FRHIGPUMask GPUMask) override
		{
			return ReadbackTexture->IsReady(GPUMask);
		}

		virtual void EnqueueCopy(FRDGBuilder& RDGBuilder, FRDGViewableResource* ResourceToReadback, bool bIsAnyThreadSupported) override
		{
			AddEnqueueCopyPass(RDGBuilder, ReadbackTexture.Get(), static_cast<FRDGTexture*>(ResourceToReadback));
		}
		//~ End FCaptureFrame interface

	public:
		/** Holds the GPU readback texture used to give a CPU buffer to the media capture callbacks. */
		TUniquePtr<FRHIGPUTextureReadback> ReadbackTexture;
	};

	/** Capture frame implementation for buffer resources. */
	class FBufferCaptureFrame : public FCaptureFrame, public TSharedFromThis<UE::MediaCaptureData::FBufferCaptureFrame, ESPMode::ThreadSafe>
	{
	public:
		/** Type alias for the output resource type used during capture frame */
		using FOutputResourceType = FRDGBufferRef;
		using PassParametersType = FMediaCaptureBufferSyncPassParameters;
	
		FBufferCaptureFrame(int32 InFrameId)
			: FCaptureFrame(InFrameId)
		{
		}

		//~ Begin FCaptureFrame Interface
		virtual bool IsTextureResource() const override
		{
			return false;
		}

		virtual bool IsBufferResource() const override
		{
			return true;
		}

		virtual void* Lock(FRHICommandListImmediate& RHICmdList, int32& OutRowStride) override
		{
			if (ReadbackBuffer->IsReady() == false)
			{
				UE_LOGF(LogMediaIOCore, Verbose, "Fence for buffer readback was not ready, blocking.");
				RHICmdList.SubmitAndBlockUntilGPUIdle();
			}

			OutRowStride = GetBufferResource()->GetStride();
			return ReadbackBuffer->Lock(GetBufferResource()->GetSize());
		}

		virtual void* Lock_Unsafe(int32& OutRowStride) override
		{
			void* ReadbackPointer = GDynamicRHI->RHILockStagingBuffer(DestinationStagingBuffers[LastCopyGPUMask.GetFirstIndex()], nullptr, 0, GetBufferResource()->GetSize());
			OutRowStride = GetBufferResource()->GetStride();
			return ReadbackPointer;
		}
	
		virtual void Unlock() override
		{
			ReadbackBuffer->Unlock();
		}
	
		virtual void Unlock_Unsafe() override
		{
			GDynamicRHI->RHIUnlockStagingBuffer(DestinationStagingBuffers[LastCopyGPUMask.GetFirstIndex()]);
		}

		virtual bool IsReadbackReady(FRHIGPUMask GPUMask) override
		{
			return ReadbackBuffer->IsReady(GPUMask);
		}
		//~ End FCaptureFrame interface

		BEGIN_SHADER_PARAMETER_STRUCT(FEnqueueCopyBufferPass, )
		RDG_BUFFER_ACCESS(Buffer, ERHIAccess::CopySrc)
		END_SHADER_PARAMETER_STRUCT()


		/** Adds a readback pass to the graph */
		virtual void EnqueueCopy(FRDGBuilder& RDGBuilder, FRDGViewableResource* ResourceToReadback, bool bIsAnyThreadSupported) override
		{
			if (bIsAnyThreadSupported)
			{
				FEnqueueCopyBufferPass* PassParameters = RDGBuilder.AllocParameters<FEnqueueCopyBufferPass>();
				PassParameters->Buffer = static_cast<FRDGBuffer*>(ResourceToReadback);

				TSharedPtr<FBufferCaptureFrame> CaptureFramePtr = AsShared();
				RDGBuilder.AddPass(
					RDG_EVENT_NAME("EnqueueCopy(%s)", ResourceToReadback->Name),
					PassParameters,
					ERDGPassFlags::Readback,
					[CaptureFramePtr, ResourceToReadback](FRHICommandList& RHICmdList)
				{
					CaptureFramePtr->LastCopyGPUMask = RHICmdList.GetGPUMask();

					for (uint32 GPUIndex : CaptureFramePtr->LastCopyGPUMask)
					{
						SCOPED_GPU_MASK(RHICmdList, FRHIGPUMask::FromIndex(GPUIndex));

						if (!CaptureFramePtr->DestinationStagingBuffers[GPUIndex])
						{
							CaptureFramePtr->DestinationStagingBuffers[GPUIndex] = RHICreateStagingBuffer();
						}

						RHICmdList.CopyToStagingBuffer(static_cast<FRHIBuffer*>(ResourceToReadback->GetRHI()), CaptureFramePtr->DestinationStagingBuffers[GPUIndex], 0, CaptureFramePtr->GetBufferResource()->GetSize());
					}
				});
			}
			else
			{
				AddEnqueueCopyPass(RDGBuilder, ReadbackBuffer.Get(), static_cast<FRDGBuffer*>(ResourceToReadback), GetBufferResource()->GetSize());
			}
		}

		//~ End FCaptureFrame Interface
	
	public:
		/** Holds the GPU readback buffer used to give a CPU buffer to the media capture callbacks. */
		TUniquePtr<FRHIGPUBufferReadback> ReadbackBuffer;
	
		// Used for the ExperimentalScheduling and anythread path
#if WITH_MGPU
		FStagingBufferRHIRef DestinationStagingBuffers[MAX_NUM_GPUS];
#else
		FStagingBufferRHIRef DestinationStagingBuffers[1];
#endif

		/** Used to keep track of the last gpu used for the readback copy. */
		FRHIGPUMask LastCopyGPUMask;
		/** Used to keep track of the last gpu used for locking the buffer. */
		uint32 LastLockGPUIndex = 0; 
	};
}
