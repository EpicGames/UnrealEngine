// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaOutput.h"

#include "MediaCapture.h"
#include "MediaIOCoreModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MediaOutput)

const FIntPoint UMediaOutput::RequestCaptureSourceSize = FIntPoint::ZeroValue;

/* UMediaOutput
 *****************************************************************************/

UMediaOutput::UMediaOutput(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, NumberOfTextureBuffers(4)
{
}

UMediaCapture* UMediaOutput::CreateMediaCapture()
{
	UMediaCapture* Result = nullptr;

	FString FailureReason;
	if (Validate(FailureReason))
	{
		Result = CreateMediaCaptureImpl();
	}
	else
	{
		UE_LOGF(LogMediaIOCore, Error, "Couldn't create the media capture. %ls", *FailureReason);
	}

	return Result;
}

bool UMediaOutput::Validate(FString& OutFailureReason) const
{
	FIntPoint RequestedSize = GetRequestedSize();
	if (RequestedSize != UMediaOutput::RequestCaptureSourceSize)
	{
		if (RequestedSize.X < 1 || RequestedSize.Y < 1)
		{
			OutFailureReason = TEXT("The requested size is invalid.");
			return false;
		}
	}

	const int32 MaxSupportedNumberOfbuffers = 8; // Arbitrary number
	if (NumberOfTextureBuffers < 1 || NumberOfTextureBuffers > MaxSupportedNumberOfbuffers)
	{
		OutFailureReason = TEXT("NumberOfTextureBuffers is not valid.");
		return false;
	}

	return true;
}

#if WITH_EDITOR
void UMediaOutput::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	UObject::PostEditChangeProperty(PropertyChangedEvent);

	OnOutputModifiedDelegate.Broadcast(this);
}
#endif

