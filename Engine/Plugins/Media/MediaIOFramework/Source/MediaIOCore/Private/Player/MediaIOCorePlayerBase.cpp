// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaIOCorePlayerBase.h"

#include "CaptureCardMediaSource.h"
#include "Async/Async.h"
#include "Engine/Engine.h"
#include "Engine/TimecodeProvider.h"
#include "GPUTextureTransferModule.h"
#include "HAL/IConsoleManager.h"
#include "IMediaEventSink.h"
#include "IMediaOptions.h"
#include "IMediaModule.h"
#include "IMediaPlayerFactory.h"
#include "ITimeManagementModule.h"
#include "MediaIOCoreAudioSampleBase.h"
#include "MediaIOCoreBinarySampleBase.h"
#include "MediaIOCoreCaptionSampleBase.h"
#include "MediaIOCoreDeinterlacer.h"
#include "MediaIOCoreModule.h"
#include "MediaIOCoreSamples.h"
#include "MediaIOCoreSubtitleSampleBase.h"
#include "MediaIOCoreTextureSampleBase.h"
#include "MediaIOCoreTextureSampleConverter.h"
#include "Misc/App.h"
#include "Misc/ScopeLock.h"
#include "RenderingThread.h"
#include "PixelFormat.h"
#include "TimedDataInputCollection.h"
#include "TimeSynchronizableMediaSource.h"

#if WITH_EDITOR
#include "EngineAnalytics.h"
#endif


#define LOCTEXT_NAMESPACE "MediaIOCorePlayerBase"

/* MediaIOCorePlayerDetail
 *****************************************************************************/

namespace MediaIOCorePlayerDetail
{
	static double ComputeTimeCodeOffset()
	{
		const FDateTime DateTime = FDateTime::Now();
		double HighPerformanceClock = FPlatformTime::Seconds();
		const FTimespan Timespan = DateTime.GetTimeOfDay();
		double Delta = Timespan.GetTotalSeconds() - HighPerformanceClock;
		return Delta;
	}

	static const double HighPerformanceClockDelta = ComputeTimeCodeOffset();
	static bool bLogTimecode = false;

	TAutoConsoleVariable<int32> CVarMediaIOMaxBufferSize(
		TEXT("MediaIO.TimedDataChannel.MaxBufferSize"),
		32,
		TEXT("The max size the MediaIO channels is allowed to set the buffer size."),
		ECVF_Default
	);

	// [JITR] Detailed insights
	static TAutoConsoleVariable<bool> CVarMediaIOJITRInsights(
		TEXT("MediaIO.JITR.Insights"),
		false,
		TEXT("Generate detailed Just-in-Time Media Rendering insights"),
		ECVF_Default);

	static FAutoConsoleCommand CCommandShowTimecode(
		TEXT("MediaIO.ShowInputTimecode"),
		TEXT("All media player will log the frame timecode when a new frame is captured."),
		FConsoleCommandDelegate::CreateLambda([](){ MediaIOCorePlayerDetail::bLogTimecode = true; }),
		ECVF_Cheat);

	static FAutoConsoleCommand CCommandHideTimecode(
		TEXT("MediaIO.HideInputTimecode"),
		TEXT("All media player will stop logging the frame timecode when a new frame is captured."),
		FConsoleCommandDelegate::CreateLambda([]() { MediaIOCorePlayerDetail::bLogTimecode = false; }),
		ECVF_Cheat);

	static TAutoConsoleVariable<int32> CVarEnableGPUDirectInput(
		TEXT("MediaIO.EnableGPUDirectInput"), 0,
		TEXT("Whether to enable GPU direct for faster video frame copies. (Experimental)"),
		ECVF_RenderThreadSafe);
}

DECLARE_GPU_STAT_NAMED(STAT_MediaIOPlayer_JITR_TransferTexture, TEXT("JITR_TransferTexture"));


/* FMediaIOCoreMediaOption structors
 *****************************************************************************/

const FName FMediaIOCoreMediaOption::FrameRateNumerator("FrameRateNumerator");
const FName FMediaIOCoreMediaOption::FrameRateDenominator("FrameRateDenominator");
const FName FMediaIOCoreMediaOption::ResolutionWidth("ResolutionWidth");
const FName FMediaIOCoreMediaOption::ResolutionHeight("ResolutionHeight");
const FName FMediaIOCoreMediaOption::VideoModeName("VideoModeName");

/* FMediaIOCorePlayerBase structors
 *****************************************************************************/

FMediaIOCorePlayerBase::FMediaIOCorePlayerBase(IMediaEventSink& InEventSink)
	: EventSink(InEventSink)
	, Samples(MakeUnique<FMediaIOCoreSamples>())
	, JITRSamples(MakeUnique<FJITRMediaTextureSamples>())
{
	FGPUTextureTransferModule& Module = FGPUTextureTransferModule::Get();
	GPUTextureTransfer = Module.GetTextureTransfer();
}

FMediaIOCorePlayerBase::~FMediaIOCorePlayerBase()
{
}

/* IMediaPlayer interface
*****************************************************************************/

void FMediaIOCorePlayerBase::Close()
{
	if (CurrentState != EMediaState::Closed)
	{
		Deinterlacer.Reset();

		CurrentState = EMediaState::Closed;
		CurrentTime = FTimespan::Zero();
		AudioTrackFormat.NumChannels = 0;
		AudioTrackFormat.SampleRate = 0;

		Samples->FlushSamples();
		JITRSamples->FlushSamples();

		UnregisterSampleBuffers();
		UnregisterTextures();

		EventSink.ReceiveMediaEvent(EMediaEvent::TracksChanged);
		EventSink.ReceiveMediaEvent(EMediaEvent::MediaClosed);
	}

	ITimeManagementModule::Get().GetTimedDataInputCollection().Remove(this);
}

FString FMediaIOCorePlayerBase::GetInfo() const
{
	FString Info;

	if (AudioTrackFormat.NumChannels > 0)
	{
		Info += FString::Printf(TEXT("Stream\n"));
		Info += FString::Printf(TEXT("    Type: Audio\n"));
		Info += FString::Printf(TEXT("    Channels: %i\n"), AudioTrackFormat.NumChannels);
		Info += FString::Printf(TEXT("    Sample Rate: %i Hz\n"), AudioTrackFormat.SampleRate);
		Info += FString::Printf(TEXT("    Bits Per Sample: 32\n"));
	}

	if (VideoTrackFormat.Dim != FIntPoint::ZeroValue)
	{
		if (!Info.IsEmpty())
		{
			Info += TEXT("\n");
		}
		Info += FString::Printf(TEXT("Stream\n"));
		Info += FString::Printf(TEXT("    Type: Video\n"));
		Info += FString::Printf(TEXT("    Dimensions: %i x %i\n"), VideoTrackFormat.Dim.X, VideoTrackFormat.Dim.Y);
		Info += FString::Printf(TEXT("    Frame Rate: %g fps\n"), VideoFrameRate.AsDecimal());
	}
	return Info;
}

IMediaCache& FMediaIOCorePlayerBase::GetCache()
{
	return *this;
}

IMediaControls& FMediaIOCorePlayerBase::GetControls()
{
	return *this;
}

IMediaSamples& FMediaIOCorePlayerBase::GetSamples()
{
	const bool bIsJITREnabled = IsJustInTimeRenderingEnabled();

	if (bIsJITREnabled)
	{
		return *JITRSamples;
	}
	else
	{
		return *Samples;
	}
}

const FMediaIOCoreSamples& FMediaIOCorePlayerBase::GetSamples() const
{
	return *Samples;
}

FString FMediaIOCorePlayerBase::GetStats() const
{
	return FString();
}

IMediaTracks& FMediaIOCorePlayerBase::GetTracks()
{
	return *this;
}

FString FMediaIOCorePlayerBase::GetUrl() const
{
	return OpenUrl;
}

IMediaView& FMediaIOCorePlayerBase::GetView()
{
	return *this;
}

bool FMediaIOCorePlayerBase::Open(const FString& Url, const IMediaOptions* Options)
{
	Close();
	CurrentState = EMediaState::Error;
	ITimeManagementModule::Get().GetTimedDataInputCollection().Add(this);

	OpenUrl = Url;
	MediaSourceObject = const_cast<UObject*>(Options->ToUObject());
	const bool bReadMediaOptions = ReadMediaOptions(Options);

	// When AutoDetect is true, we can't create textures now as we don't know the video format.
	// In this case we need to wait until auto-detection is finished, and video format is known.
	if (!bAutoDetect && CanUseGPUTextureTransfer())
	{
		CreateAndRegisterTextures();
	}
	
	const EMediaIOInterlaceFieldOrder InterlaceFieldOrder = static_cast<EMediaIOInterlaceFieldOrder>(Options->GetMediaOption(UE::CaptureCardMediaSource::InterlaceFieldOrder, (int64)EMediaIOInterlaceFieldOrder::TopFieldFirst));
	const FString DefaultOption = TEXT("");
	const FSoftObjectPath DeinterlacerPath = FSoftObjectPath(Options->GetMediaOption(UE::CaptureCardMediaSource::Deinterlacer, DefaultOption));
	
	UObject* DeinterlacerInstancer = DeinterlacerPath.ResolveObject();

	TSharedPtr<UE::MediaIOCore::FDeinterlacer> DeinterlacerInstance;
	if (DeinterlacerInstancer && DeinterlacerInstancer->IsA<UVideoDeinterlacer>())
	{
		Deinterlacer = Cast<UVideoDeinterlacer>(DeinterlacerInstancer)->Instantiate(UE::MediaIOCore::FDeinterlacer::FOnAcquireSample_AnyThread::CreateSP(this, &FMediaIOCorePlayerBase::AcquireTextureSample_AnyThread), InterlaceFieldOrder);
	}
	else
	{
		Deinterlacer = MakeShared<UE::MediaIOCore::FDeinterlacer>(UE::MediaIOCore::FDeinterlacer::FOnAcquireSample_AnyThread::CreateSP(this, &FMediaIOCorePlayerBase::AcquireTextureSample_AnyThread), InterlaceFieldOrder);
	}

	if (bReadMediaOptions)
	{
		CurrentState = EMediaState::Preparing;
		EventSink.ReceiveMediaEvent(EMediaEvent::MediaConnecting);
	}

	return bReadMediaOptions;
}

bool FMediaIOCorePlayerBase::Open(const TSharedRef<FArchive, ESPMode::ThreadSafe>& /*Archive*/, const FString& /*OriginalUrl*/, const IMediaOptions* /*Options*/)
{
	return false;
}

void FMediaIOCorePlayerBase::TickTimeManagement()
{
	if (EvaluationType == EMediaIOSampleEvaluationType::Timecode)
	{
		TOptional<FQualifiedFrameTime> CurrentFrameTime = FApp::GetCurrentFrameTime();
		if (CurrentFrameTime.IsSet())
		{
			if (!bWarnedIncompatibleFrameRate)
			{
				if (!CurrentFrameTime->Rate.IsMultipleOf(VideoFrameRate) && !VideoFrameRate.IsMultipleOf(CurrentFrameTime->Rate))
				{
					bWarnedIncompatibleFrameRate = true;
					UE_LOGF(LogMediaIOCore, Warning, "The video's frame rate is incompatible with engine's frame rate. %ls", *OpenUrl);
				}
			}

			if (FrameDelay != 0)
			{
				FFrameTime FrameNumberVideoRate = CurrentFrameTime->ConvertTo(VideoFrameRate);
				FrameNumberVideoRate.FrameNumber -= FrameDelay;
				CurrentFrameTime->Time = FFrameRate::TransformTime(FrameNumberVideoRate, VideoFrameRate, CurrentFrameTime->Rate);
			}

			CurrentTime = FTimespan::FromSeconds(CurrentFrameTime->AsSeconds());
		}
		else
		{
			UE_LOGF(LogMediaIOCore, Warning, "The video '%ls' is configured to use timecode but none is available on the engine.", *OpenUrl);
		}
	}
	else
	{
		// As default, use the App time
		CurrentTime = FTimespan::FromSeconds(GetApplicationSeconds() - TimeDelay);
	}
}

void FMediaIOCorePlayerBase::TickFetch(FTimespan DeltaTime, FTimespan Timecode)
{
	// Running JITR?
	if (CurrentState == EMediaState::Playing && IsJustInTimeRenderingEnabled())
	{
		// Running in JITR mode implies that the sample most likely is going to arrive closer to the render thread / RHI thread command execution.
		// AcquireJITRProxySampleInitialized is called on game thread before any rendering commands are executed for this frame.
		// With just in time enabled, the Sample list could be empty and the sample could arrive much later. 
		// The inheriting players must be responsible for handling the provision of JIT samples at the later stage.
		JITRSamples->ProxySample = AcquireJITRProxySampleInitialized();
	}

	Samples->CacheSamplesState(GetTime());
}

/* IMediaCache interface
 *****************************************************************************/
bool FMediaIOCorePlayerBase::QueryCacheState(EMediaCacheState State, TRangeSet<FTimespan>& OutTimeRanges) const
{
	if (!Samples || Samples->NumVideoSamples() <= 0)
	{
		return false;
	}
	
	bool bHasQueried = false;
	if (State == EMediaCacheState::Loaded)
	{
		const FTimespan FrameDuration = FTimespan::FromSeconds(VideoFrameRate.AsInterval());
		const FTimespan NextSampleTime = Samples->GetNextVideoSampleTime();
		OutTimeRanges.Add(TRange<FTimespan>(NextSampleTime, NextSampleTime + FrameDuration * Samples->NumVideoSamples()));
		bHasQueried = true;
	}

	return bHasQueried;
}

int32 FMediaIOCorePlayerBase::GetSampleCount(EMediaCacheState State) const
{
	int32 Count = 0;
	if (State == EMediaCacheState::Loaded)
	{
		if (Samples)
		{
			Count = Samples->NumVideoSamples();
		}
	}

	return Count;
}

/* IMediaControls interface
 *****************************************************************************/

bool FMediaIOCorePlayerBase::CanControl(EMediaControl Control) const
{
	return false;
}


FTimespan FMediaIOCorePlayerBase::GetDuration() const
{
	return (CurrentState == EMediaState::Playing) ? FTimespan::MaxValue() : FTimespan::Zero();
}


float FMediaIOCorePlayerBase::GetRate() const
{
	return (CurrentState == EMediaState::Playing) ? 1.0f : 0.0f;
}


EMediaState FMediaIOCorePlayerBase::GetState() const
{
	return CurrentState;
}


EMediaStatus FMediaIOCorePlayerBase::GetStatus() const
{
	return (CurrentState == EMediaState::Preparing) ? EMediaStatus::Connecting : EMediaStatus::None;
}

TRangeSet<float> FMediaIOCorePlayerBase::GetSupportedRates(EMediaRateThinning /*Thinning*/) const
{
	TRangeSet<float> Result;
	Result.Add(TRange<float>(1.0f));
	return Result;
}

FTimespan FMediaIOCorePlayerBase::GetTime() const
{
	return CurrentTime;
}


bool FMediaIOCorePlayerBase::IsLooping() const
{
	return false; // not supported
}


bool FMediaIOCorePlayerBase::Seek(const FTimespan& Time)
{
	return false; // not supported
}


bool FMediaIOCorePlayerBase::SetLooping(bool Looping)
{
	return false; // not supported
}


bool FMediaIOCorePlayerBase::SetRate(float Rate)
{
	//Return true if a proper rate is applied. 
	return FMath::IsNearlyEqual(Rate, 1.0f);
}


/* IMediaTracks interface
 *****************************************************************************/

bool FMediaIOCorePlayerBase::GetAudioTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaAudioTrackFormat& OutFormat) const
{
	if (!IsHardwareReady() || TrackIndex != 0 || FormatIndex != 0)
	{
		return false;
	}

	OutFormat = AudioTrackFormat;
	return true;
}


int32 FMediaIOCorePlayerBase::GetNumTracks(EMediaTrackType TrackType) const
{
	return 1;
}

int32 FMediaIOCorePlayerBase::GetNumTrackFormats(EMediaTrackType TrackType, int32 TrackIndex) const
{
	return 1;
}


int32 FMediaIOCorePlayerBase::GetSelectedTrack(EMediaTrackType TrackType) const
{
	switch (TrackType)
	{
	case EMediaTrackType::Audio:
	case EMediaTrackType::Video:
		return 0;

	default:
		return INDEX_NONE;
	}
}


FText FMediaIOCorePlayerBase::GetTrackDisplayName(EMediaTrackType TrackType, int32 TrackIndex) const
{
	if (!IsHardwareReady() || TrackIndex != 0)
	{
		return FText::GetEmpty();
	}

	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		return LOCTEXT("DefaultAudioTrackName", "Audio Track");

	case EMediaTrackType::Video:
		return LOCTEXT("DefaultVideoTrackName", "Video Track");

	default:
		break;
	}

	return FText::GetEmpty();
}


int32 FMediaIOCorePlayerBase::GetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex) const
{
	if (TrackType == EMediaTrackType::Video) {
		return 0;
	}
	return INDEX_NONE;
}


FString FMediaIOCorePlayerBase::GetTrackLanguage(EMediaTrackType TrackType, int32 TrackIndex) const
{
	return FString();
}


FString FMediaIOCorePlayerBase::GetTrackName(EMediaTrackType TrackType, int32 TrackIndex) const
{
	return FString();
}


bool FMediaIOCorePlayerBase::GetVideoTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaVideoTrackFormat& OutFormat) const
{
	if (!IsHardwareReady() || TrackIndex != 0 || FormatIndex != 0)
	{
		return false;
	}

	OutFormat = VideoTrackFormat;
	return true;
}


bool FMediaIOCorePlayerBase::SelectTrack(EMediaTrackType TrackType, int32 TrackIndex)
{
	if (!IsHardwareReady() || TrackIndex < INDEX_NONE || TrackIndex != 0)
	{
		return false;
	}

	// Only 1 track supported
	return (TrackType == EMediaTrackType::Audio || TrackType == EMediaTrackType::Video);
}

bool FMediaIOCorePlayerBase::SetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex, int32 FormatIndex)
{
	return false;
}

bool FMediaIOCorePlayerBase::ReadMediaOptions(const IMediaOptions* Options)
{
	// Break dependency on the bUseTimeSynchronization option, and use EvaluationType instead
	const bool bUseTimeSynchronization = Options->GetMediaOption(TimeSynchronizableMedia::UseTimeSynchronizatioOption, false);

	FrameDelay = Options->GetMediaOption(TimeSynchronizableMedia::FrameDelay, (int64)0);
	TimeDelay = Options->GetMediaOption(TimeSynchronizableMedia::TimeDelay, 0.0);
	bAutoDetect = Options->GetMediaOption(TimeSynchronizableMedia::AutoDetect, false);

	EvaluationType = (EMediaIOSampleEvaluationType)(Options->GetMediaOption(UE::CaptureCardMediaSource::EvaluationType, (int64)EMediaIOSampleEvaluationType::PlatformTime));
	bJustInTimeRender = Options->GetMediaOption(UE::CaptureCardMediaSource::RenderJIT, true);
	bFramelock = Options->GetMediaOption(UE::CaptureCardMediaSource::Framelock, false);

	{
		int32 Numerator = Options->GetMediaOption(FMediaIOCoreMediaOption::FrameRateNumerator, (int64)30);
		int32 Denominator = Options->GetMediaOption(FMediaIOCoreMediaOption::FrameRateDenominator, (int64)1);
		VideoFrameRate = FFrameRate(Numerator, Denominator);
	}
	{
		int32 ResolutionX = Options->GetMediaOption(FMediaIOCoreMediaOption::ResolutionWidth, (int64)1920);
		int32 ResolutionY = Options->GetMediaOption(FMediaIOCoreMediaOption::ResolutionHeight, (int64)1080);
		VideoTrackFormat.Dim = FIntPoint(ResolutionX, ResolutionY);
		VideoTrackFormat.FrameRates = TRange<float>(VideoFrameRate.AsDecimal());
		VideoTrackFormat.FrameRate = VideoFrameRate.AsDecimal();
		VideoTrackFormat.TypeName = Options->GetMediaOption(FMediaIOCoreMediaOption::VideoModeName, FString(TEXT("1080p30")));
	}

	// Make sure we haven't got invalid data from media source. When time sync is off,
	// timecode based evaluation and framelock are unavailable.
	if (!bUseTimeSynchronization)
	{
		if (EvaluationType == EMediaIOSampleEvaluationType::Timecode)
		{
			EvaluationType = EMediaIOSampleEvaluationType::PlatformTime;
		}
		
		bFramelock = false;
	}

	//Setup base sampling settings
	BaseSettings.EvaluationType = GetEvaluationType(); // bJustInTimeRender and JITREvaluationType should have been set before calling
	BaseSettings.FrameRate = VideoFrameRate;
	BaseSettings.PlayerTimeOffset = MediaIOCorePlayerDetail::HighPerformanceClockDelta;
	BaseSettings.AbsoluteMaxBufferSize = MediaIOCorePlayerDetail::CVarMediaIOMaxBufferSize.GetValueOnGameThread();

	TSharedPtr<IMediaOptions::FDataContainer, ESPMode::ThreadSafe> DefaultOption;
	const TSharedPtr<UCaptureCardMediaSource::FOpenColorIODataContainer> OCIODataContainer = StaticCastSharedPtr<UCaptureCardMediaSource::FOpenColorIODataContainer>(Options->GetMediaOption(UE::CaptureCardMediaSource::OpenColorIOSettings, DefaultOption));
	if (OCIODataContainer)
	{
		OCIOSettings = MakeShared<FOpenColorIOColorConversionSettings>(OCIODataContainer->ColorConversionSettings);
	}

	TSharedPtr<IMediaOptions::FDataContainer, ESPMode::ThreadSafe> DataContainer = Options->GetMediaOption(UE::CaptureCardMediaSource::SourceColorSettingsOption, DefaultOption);
	if (DataContainer.IsValid())
	{
		SourceColorSettings = StaticCastSharedPtr<FNativeMediaSourceColorSettings, IMediaOptions::FDataContainer, ESPMode::ThreadSafe>(DataContainer);
	}

	return true;
}

void FMediaIOCorePlayerBase::NotifyVideoFormatDetected()
{
	// Now that we know the video format, we can initialize the textures
	if (bAutoDetect && CanUseGPUTextureTransfer())
	{
		CreateAndRegisterTextures();
	}
}

double FMediaIOCorePlayerBase::GetApplicationSeconds()
{
	return MediaIOCorePlayerDetail::HighPerformanceClockDelta + FApp::GetCurrentTime();
}

double FMediaIOCorePlayerBase::GetPlatformSeconds()
{
	return MediaIOCorePlayerDetail::HighPerformanceClockDelta + FPlatformTime::Seconds();
}

bool FMediaIOCorePlayerBase::IsTimecodeLogEnabled()
{
#if !(UE_BUILD_SHIPPING)
	return MediaIOCorePlayerDetail::bLogTimecode;
#else
	return false;
#endif
}

bool FMediaIOCorePlayerBase::CanUseGPUTextureTransfer() const
{
	return GPUTextureTransfer && MediaIOCorePlayerDetail::CVarEnableGPUDirectInput.GetValueOnAnyThread();
}

bool FMediaIOCorePlayerBase::HasTextureAvailableForGPUTransfer() const
{
	FScopeLock ScopeLock(&TexturesCriticalSection);
	return !Textures.IsEmpty();
}

bool FMediaIOCorePlayerBase::IsJustInTimeRenderingEnabled() const
{
	return bJustInTimeRender;
}

void FMediaIOCorePlayerBase::AddVideoSampleAfterGPUTransfer_RenderThread(const TSharedRef<FMediaIOCoreTextureSampleBase>& Sample)
{
	checkSlow(IsInRenderingThread());

	LogBookmark(TEXT("MIO Tex Copy"), Sample);

	if (!IsJustInTimeRenderingEnabled())
	{
		Samples->AddVideo(Sample);
	}
}

void FMediaIOCorePlayerBase::OnSampleDestroyed(TRefCountPtr<FRHITexture> InTexture)
{
	FScopeLock ScopeLock(&TexturesCriticalSection);
	if (InTexture && RegisteredTextures.Contains(InTexture))
	{
		Textures.Add(InTexture);
	}
}

void FMediaIOCorePlayerBase::RegisterSampleBuffer(const TSharedPtr<FMediaIOCoreTextureSampleBase>& InSample)
{
	if (!RegisteredBuffers.Contains(InSample->GetMutableBuffer()))
	{
		RegisteredBuffers.Add(InSample->GetMutableBuffer());

		// Enqueue buffer registration on the render  thread to ensure it happens before the call to TransferTexture
		ENQUEUE_RENDER_COMMAND(RegisterSampleBuffers)([this, InSample](FRHICommandListImmediate&)
		{
			if (!InSample->GetMutableBuffer())
			{
				UE_LOGF(LogMediaIOCore, Error, "A buffer was not available while performing a gpu texture transfer.");
				return;
			}

			if (!InSample->GetTexture())
			{
				UE_LOGF(LogMediaIOCore, Error, "A texture was not available while performing a gpu texture transfer.");
				return;
			}

			const uint32 TextureWidth = InSample->GetTexture()->GetDesc().Extent.X;
			const uint32 TextureHeight = InSample->GetTexture()->GetDesc().Extent.Y;
			uint32 TextureStride = TextureWidth * 4;

			EPixelFormat Format = InSample->GetTexture()->GetFormat();
			if (Format == PF_R32G32B32A32_UINT)
			{
				TextureStride *= 4;
			}

			UE_LOGF(LogMediaIOCore, Verbose, "Registering buffer %llu", reinterpret_cast<uintptr_t>(InSample->GetMutableBuffer()));
			UE::GPUTextureTransfer::FRegisterDMABufferArgs Args;
			Args.Buffer = InSample->GetMutableBuffer();
			Args.Width = TextureWidth;
			Args.Height = TextureHeight;
			Args.Stride = TextureStride;
			Args.PixelFormat = Format == PF_B8G8R8A8 ? UE::GPUTextureTransfer::EPixelFormat::PF_8Bit : UE::GPUTextureTransfer::EPixelFormat::PF_10Bit;
			GPUTextureTransfer->RegisterBuffer(Args);
		});
	}
	else
	{
		UE_LOGF(LogMediaIOCore, VeryVerbose, "Buffer already registered: %llu", reinterpret_cast<uintptr_t>(InSample->GetMutableBuffer()));
	}
}

void FMediaIOCorePlayerBase::UnregisterSampleBuffers()
{
	ENQUEUE_RENDER_COMMAND(UnregisterSampleBuffers)([TextureTransfer = GPUTextureTransfer, BuffersToUnregister = RegisteredBuffers](FRHICommandListImmediate&)
	{
		if (TextureTransfer)
		{
			for (void* RegisteredBuffer : BuffersToUnregister)
			{
				TextureTransfer->UnregisterBuffer(RegisteredBuffer);
			}
		}
	});

	RegisteredBuffers.Reset();
}

void FMediaIOCorePlayerBase::CreateAndRegisterTextures()
{
	// Call this on the game thread so we can stall until textures are created
	if (!IsInGameThread())
	{
		AsyncTask(ENamedThreads::GameThread, [this]()
			{
				CreateAndRegisterTextures();
			});

		return;
	}


	UnregisterTextures();

	ENQUEUE_RENDER_COMMAND(MediaIOCorePlayerCreateTextures)(
		[this](FRHICommandListImmediate& CommandList)
		{
			if (!GPUTextureTransfer)
			{
				UE_LOGF(LogMediaIOCore, Error, "Could not register textures for gpu texture transfer, as the GPUTextureTransfer object was null.");
				return;
			}

			// We need only one texture for JIT rendering. However the original non-JITR pipeline
			// requires the number of textures must not be less than the number of buffers
			const bool   bIsJustInTimeRenderingEnabled = IsJustInTimeRenderingEnabled();
			const uint32 VideoFrameBuffersAmount = GetNumVideoFrameBuffers();
			const uint32 TexturesAmount = bIsJustInTimeRenderingEnabled ? 1 : VideoFrameBuffersAmount;

			RegisteredTextures.Reserve(TexturesAmount);
			Textures.Reserve(TexturesAmount);

			for (uint8 Index = 0; Index < TexturesAmount; Index++)
			{
				TRefCountPtr<FRHITexture> RHITexture;
				const FString TextureName = FString::Printf(TEXT("FMediaIOCorePlayerTexture %d"), Index);

				uint32 TextureWidth = VideoTrackFormat.Dim.X;
				const uint32 TextureHeight = VideoTrackFormat.Dim.Y;
				uint32 Stride = TextureWidth;
				EPixelFormat InputFormat = EPixelFormat::PF_Unknown;

				EMediaIOCoreColorFormat ColorFormat = GetColorFormat();

				if (ColorFormat == EMediaIOCoreColorFormat::YUV8)
				{
					InputFormat = EPixelFormat::PF_B8G8R8A8;
					TextureWidth /= 2;
					Stride = TextureWidth * 4;
				}
				else if (ColorFormat == EMediaIOCoreColorFormat::YUV10)
				{
					InputFormat = EPixelFormat::PF_A2B10G10R10;
					TextureWidth = 4 * (TextureWidth / 6);
					Stride = TextureWidth * 4;
				}
				else
				{
					checkf(false, TEXT("Format not supported"));
				}

				ETextureCreateFlags CreateFlags = ETextureCreateFlags::Shared;
				if (RHIGetInterfaceType() == ERHIInterfaceType::Vulkan)
				{
					CreateFlags = ETextureCreateFlags::External;
				}

				FRHITextureCreateDesc CreateDesc = FRHITextureCreateDesc::Create2D(*TextureName)
					.SetExtent(TextureWidth, TextureHeight)
					.SetFormat(InputFormat)
					.SetClearValue(FClearValueBinding::White)
					.SetFlags(CreateFlags | ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ShaderResource)
					.SetInitialState(ERHIAccess::SRVMask);

				RHITexture = CommandList.CreateTexture(CreateDesc);

				UE_LOGF(LogMediaIOCore, Verbose, "Registering texture %llu", reinterpret_cast<uintptr_t>(RHITexture->GetNativeResource()));

				UE::GPUTextureTransfer::FRegisterDMATextureArgs Args;
				Args.RHITexture = RHITexture.GetReference();
				Args.Stride = Stride;
				Args.Width = TextureWidth;
				Args.Height = TextureHeight;
				Args.RHIResourceMemory = nullptr;
				Args.PixelFormat = ColorFormat == EMediaIOCoreColorFormat::YUV8 ? UE::GPUTextureTransfer::EPixelFormat::PF_8Bit : UE::GPUTextureTransfer::EPixelFormat::PF_10Bit;

				GPUTextureTransfer->RegisterTexture(Args);
				RegisteredTextures.Add(RHITexture);

				{
					FScopeLock Lock(&TexturesCriticalSection);
					Textures.Add(RHITexture);
				}
			}
		});

	FRenderCommandFence RenderFence;
	RenderFence.BeginFence();
	RenderFence.Wait();
}

void FMediaIOCorePlayerBase::UnregisterTextures()
{
	// Keep a copy of the textures to make sure they were not destoryed before this gets executed.
	ENQUEUE_RENDER_COMMAND(UnregisterSampleBuffers)([TextureTransfer = GPUTextureTransfer, TexturesToUnregister = RegisteredTextures](FRHICommandList& CommandList)
		{
			if (TextureTransfer)
			{
				for (const TRefCountPtr<FRHITexture>& RHITexture : TexturesToUnregister)
				{
					UE_LOGF(LogMediaIOCore, Verbose, "Unregistering texture %llu", reinterpret_cast<uintptr_t>(RHITexture->GetNativeResource()));
					TextureTransfer->UnregisterTexture(RHITexture.GetReference());
				}
			}
		});

	RegisteredTextures.Reset();

	{
		FScopeLock Lock(&TexturesCriticalSection);
		Textures.Reset();
	}
}

void FMediaIOCorePlayerBase::PreGPUTransfer(const TSharedPtr<FMediaIOCoreTextureSampleBase>& InSample)
{
	TRefCountPtr<FRHITexture> Texture;

	// Get a texture from the pool
	{
		FScopeLock Lock(&TexturesCriticalSection);

		const bool bHasTexture = !Textures.IsEmpty();
		ensureMsgf(bHasTexture, TEXT("No texture available while doing a gpu texture transfer."));

		if (bHasTexture)
		{
			Texture = Textures.Pop();
		}
	}

	// Register the buffer and the texture
	if (Texture && RegisteredTextures.Contains(Texture))
	{
		InSample->SetTexture(Texture);
		InSample->SetDestructionCallback([MediaPlayerWeakPtr = TWeakPtr<FMediaIOCorePlayerBase>(AsShared())](TRefCountPtr<FRHITexture> InTexture)
		{
			if (TSharedPtr<FMediaIOCorePlayerBase> MediaPlayerPtr = MediaPlayerWeakPtr.Pin())
			{
				MediaPlayerPtr->OnSampleDestroyed(InTexture);
			}
		});

		RegisterSampleBuffer(InSample);
	}
	else
	{
		UE_LOGF(LogMediaIOCore, Display, "Unregistered texture %llu encountered while doing a gpu texture transfer.", Texture ? reinterpret_cast<uintptr_t>(Texture->GetNativeResource()) : 0);
	}
}

void FMediaIOCorePlayerBase::PreGPUTransferJITR(const TSharedPtr<FMediaIOCoreTextureSampleBase>& InSample, const TSharedPtr<FMediaIOCoreTextureSampleBase>& InJITRProxySample)
{
	TRefCountPtr<FRHITexture> Texture;

	// Get a texture from the pool. JITR path uses a single texture for GPU transfer.
	{
		FScopeLock Lock(&TexturesCriticalSection);

		const bool bHasTexture = !Textures.IsEmpty();

		// Normally this should never yell as we should always have a single texture
		ensureMsgf(bHasTexture, TEXT("No texture available while doing a gpu texture transfer."));

		Texture = Textures[0];
	}

	// Register the buffer and the texture
	if (Texture && RegisteredTextures.Contains(Texture))
	{
		// Set data that is necessary to perform fast texture copy right to the proxy sample
		InJITRProxySample->SetTexture(Texture);
		InJITRProxySample->SetBuffer(InSample->GetMutableBuffer());

		RegisterSampleBuffer(InJITRProxySample);
	}
	else
	{
		UE_LOGF(LogMediaIOCore, Display, "Unregistered texture %llu encountered while doing a gpu texture transfer.", Texture ? reinterpret_cast<uintptr_t>(Texture->GetNativeResource()) : 0);
	}
}

void FMediaIOCorePlayerBase::ExecuteGPUTransfer(const TSharedPtr<FMediaIOCoreTextureSampleBase>& InSample)
{
	ENQUEUE_RENDER_COMMAND(StartGPUTextureTransferCopy)(
		[MediaPlayerWeakPtr = TWeakPtr<FMediaIOCorePlayerBase>(AsShared()), InSample](FRHICommandListImmediate& RHICmdList)
	{
		if (InSample->GetTexture())
		{
			if (TSharedPtr<FMediaIOCorePlayerBase> MediaPlayerPtr = MediaPlayerWeakPtr.Pin();
				MediaPlayerPtr != nullptr && MediaPlayerPtr->GPUTextureTransfer != nullptr)
			{
				void* Buffer = InSample->GetMutableBuffer();
				UE_LOGF(LogMediaIOCore, Verbose, "Starting Transfer with buffer %llu", reinterpret_cast<uintptr_t>(Buffer));
				MediaPlayerPtr->GPUTextureTransfer->TransferTexture(Buffer, InSample->GetTexture(), UE::GPUTextureTransfer::ETransferDirection::CPU_TO_GPU);
				MediaPlayerPtr->GPUTextureTransfer->BeginSync((InSample)->GetMutableBuffer(), UE::GPUTextureTransfer::ETransferDirection::CPU_TO_GPU);
				MediaPlayerPtr->AddVideoSampleAfterGPUTransfer_RenderThread(InSample.ToSharedRef());
				MediaPlayerPtr->GPUTextureTransfer->EndSync(InSample->GetMutableBuffer());
			}
		}
	});
}

FMediaIOCoreSamples& FMediaIOCorePlayerBase::GetSamples_Internal()
{
	if (IsJustInTimeRenderingEnabled())
	{
		return *JITRSamples;
	}
	return *Samples;
}

TArray<TSharedRef<FMediaIOCoreTextureSampleBase>> FMediaIOCorePlayerBase::Deinterlace(const UE::MediaIOCore::FVideoFrame& InVideoFrame) const
{
	if (Deinterlacer)
	{
		return Deinterlacer->Deinterlace(InVideoFrame);
	}
	return {};
}

TArray<ITimedDataInputChannel*> FMediaIOCorePlayerBase::GetChannels() const
{
	return Channels;
}

FText FMediaIOCorePlayerBase::GetDisplayName() const
{
	IMediaModule* MediaModule = FModuleManager::Get().GetModulePtr<IMediaModule>("Media");
	if (!MediaModule)
	{
		return LOCTEXT("InvalidMediaModule", "NoMediaModule");
	}

	IMediaPlayerFactory* PlayerFactory = MediaModule->GetPlayerFactory(GetPlayerPluginGUID());
	if (PlayerFactory == nullptr)
	{
		return LOCTEXT("InvalidPlayerFactory", "NoPlayerFactory");
	}

	return FText::Format(LOCTEXT("PlayerDisplayName", "{0} - {1}"), FText::FromName(PlayerFactory->GetPlayerName()), FText::FromString(GetUrl()));
}

ETimedDataInputEvaluationType FMediaIOCorePlayerBase::GetEvaluationType() const
{
	switch (EvaluationType)
	{
	case EMediaIOSampleEvaluationType::Latest:
		return ETimedDataInputEvaluationType::None;

	case EMediaIOSampleEvaluationType::PlatformTime:
		return ETimedDataInputEvaluationType::PlatformTime;

	case EMediaIOSampleEvaluationType::Timecode:
		return ETimedDataInputEvaluationType::Timecode;

	default:
		checkNoEntry();
		return ETimedDataInputEvaluationType::None;
	}
}

void FMediaIOCorePlayerBase::SetEvaluationType(ETimedDataInputEvaluationType Evaluation)
{
	bool bNewEvaluationModeSet = false;

	switch (Evaluation)
	{
	case ETimedDataInputEvaluationType::None:
		bNewEvaluationModeSet = (EvaluationType != EMediaIOSampleEvaluationType::Latest);
		EvaluationType = EMediaIOSampleEvaluationType::Latest;
		break;

	case ETimedDataInputEvaluationType::PlatformTime:
		bNewEvaluationModeSet = (EvaluationType != EMediaIOSampleEvaluationType::PlatformTime);
		EvaluationType = EMediaIOSampleEvaluationType::PlatformTime;
		break;

	case ETimedDataInputEvaluationType::Timecode:
		bNewEvaluationModeSet = (EvaluationType != EMediaIOSampleEvaluationType::Timecode);
		EvaluationType = EMediaIOSampleEvaluationType::Timecode;
		break;

	default:
		checkNoEntry();
		break;
	}

	if (bNewEvaluationModeSet)
	{
		BaseSettings.EvaluationType = GetEvaluationType();
		SetupSampleChannels();

		if (UCaptureCardMediaSource* CaptureSource = Cast<UCaptureCardMediaSource>(MediaSourceObject.Get()))
		{
			CaptureSource->EvaluationType = EvaluationType;
			if (EvaluationType == EMediaIOSampleEvaluationType::Timecode)
			{
				CaptureSource->bUseTimeSynchronization = true;
			}
		}
	}
}

double FMediaIOCorePlayerBase::GetEvaluationOffsetInSeconds() const
{
	switch (EvaluationType)
	{
	case EMediaIOSampleEvaluationType::Latest:
		return 0;

	case EMediaIOSampleEvaluationType::PlatformTime:
		return TimeDelay;

	case EMediaIOSampleEvaluationType::Timecode:
		return ITimedDataInput::ConvertFrameOffsetInSecondOffset(FrameDelay, VideoFrameRate);

	default:
		checkNoEntry();
		return 0;
	}
}

void FMediaIOCorePlayerBase::SetEvaluationOffsetInSeconds(double Offset)
{
	switch (EvaluationType)
	{
	case EMediaIOSampleEvaluationType::Latest:
		break;

	case EMediaIOSampleEvaluationType::PlatformTime:
		TimeDelay = Offset;
		break;

	case EMediaIOSampleEvaluationType::Timecode:
	{
		const double FrameOffset = ITimedDataInput::ConvertSecondOffsetInFrameOffset(Offset, VideoFrameRate);
		FrameDelay = FMath::CeilToInt(FrameOffset);
		break;
	}

	default:
		checkNoEntry();
		break;
	}

	if (UTimeSynchronizableMediaSource* Source = Cast<UTimeSynchronizableMediaSource>(MediaSourceObject.Get()))
	{
		Source->FrameDelay = FrameDelay;
		Source->TimeDelay = TimeDelay;
	}
}

FFrameRate FMediaIOCorePlayerBase::GetFrameRate() const
{
	return VideoFrameRate;
}

bool FMediaIOCorePlayerBase::IsDataBufferSizeControlledByInput() const
{
	//Each channel (audio, video, etc...) can have a different size
	return false;
}

void FMediaIOCorePlayerBase::AddChannel(ITimedDataInputChannel* Channel)
{
	Channels.AddUnique(Channel);
}

void FMediaIOCorePlayerBase::RemoveChannel(ITimedDataInputChannel* Channel)
{
	Channels.Remove(Channel);
}

bool FMediaIOCorePlayerBase::SupportsSubFrames() const
{
	return false;
}

UObject* FMediaIOCorePlayerBase::GetSettingsObject() const
{
	return MediaSourceObject.Get();
}

void FMediaIOCorePlayerBase::AddAudioSample(const TSharedRef<FMediaIOCoreAudioSampleBase>& Sample)
{
	GetSamples_Internal().AddAudio(Sample);
}

void FMediaIOCorePlayerBase::AddCaptionSample(const TSharedRef<FMediaIOCoreCaptionSampleBase>& Sample)
{
	GetSamples_Internal().AddCaption(Sample);
}

void FMediaIOCorePlayerBase::AddMetadataSample(const TSharedRef<FMediaIOCoreBinarySampleBase>& Sample)
{
	GetSamples_Internal().AddMetadata(Sample);
}

void FMediaIOCorePlayerBase::AddSubtitleSample(const TSharedRef<FMediaIOCoreSubtitleSampleBase>& Sample)
{
	GetSamples_Internal().AddSubtitle(Sample);
}

void FMediaIOCorePlayerBase::AddVideoSample(const TSharedRef<FMediaIOCoreTextureSampleBase>& Sample)
{
	LogBookmark(TEXT("MIO in"), Sample);

	// When JITR is used, we postpone texture transferring until this sample is chosen to render. It's also
	// possible that this sample will never be rendered, therefore we don't waste resources for transferring.
	if (IsJustInTimeRenderingEnabled())
	{
		Samples->AddVideo(Sample);
	}
	// Otherwise we run original non-JITR pipeline which implies immediate texture transfer if GPUDirect path is available.
	else
	{
		const bool bCanUseGPUTextureTransfer = CanUseGPUTextureTransfer();
		const bool bIsAwaitingForGPUTransfer = Sample->IsAwaitingForGPUTransfer();

		// So, when GPUDirect is available we transfer data on the rendering thread. The callback AddVideoSampleAfterGPUTransfer_RenderThread
		// will put the sample to the pool when finished.
		if (bCanUseGPUTextureTransfer && bIsAwaitingForGPUTransfer)
		{
			PreGPUTransfer(Sample);
			ExecuteGPUTransfer(Sample);
		}
		// Just push the sample to the pool if fast GPU transfer isn't available
		else
		{
			Samples->AddVideo(Sample);
		}
	}
}

bool FMediaIOCorePlayerBase::JustInTimeSampleRender_RenderThread(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDestinationTexture, TSharedPtr<FMediaIOCoreTextureSampleBase>& JITRProxySample)
{
	checkSlow(IsInRenderingThread());

	// Make sure the sample is valid
	if (!JITRProxySample.IsValid())
	{
		UE_LOGF(LogMediaIOCore, Warning, "Invalid JITR sample.");
		return false;
	}

	// JIT rendering is performed once per frame
	if (LastEngineRTFrameThatUpdatedJustInTime == GFrameCounterRenderThread)
	{
		return false;
	}

	// Pick a sample to render
	FFrameInfo FrameInformation;
	FrameInformation.RequestedTimecode = JITRProxySample->GetTimecode().Get(FTimecode());
	FrameInformation.SampleTimespan = JITRProxySample->GetTime().Time;
	FrameInformation.EvaluationOffset = JITRProxySample->GetEvaluationOffsetInSeconds();

	TSharedPtr<FMediaIOCoreTextureSampleBase> SourceSample = PickSampleToRender_RenderThread(FrameInformation);

	if (!SourceSample)
	{
		UE_LOGF(LogMediaIOCore, Warning, "JustInTimeSampleRender couldn't find a sample to render");
		return false;
	}

	LogBookmark(TEXT("JITR out"), SourceSample.ToSharedRef());

	// Update proxy sample configuration
	JITRProxySample->CopyConfiguration(SourceSample);

	// Now we know which sample to use, transfer its texture
	TransferTexture_RenderThread(RHICmdList, SourceSample, JITRProxySample);

	// Update frame number so we won't render this sample again
	LastEngineRTFrameThatUpdatedJustInTime = GFrameCounterRenderThread;

	return true;
}

TSharedPtr<FMediaIOCoreTextureSampleConverter> FMediaIOCorePlayerBase::CreateTextureSampleConverter() const
{
	return MakeShared<FMediaIOCoreTextureSampleConverter>();
}

TSharedPtr<FMediaIOCoreTextureSampleBase> FMediaIOCorePlayerBase::AcquireJITRProxySampleInitialized()
{
	// Create a new media sample acting as a dummy container to be picked by MFW which we will fill during the late update
	FMediaIOCoreSampleJITRConfigurationArgs Args;
	Args.Width     = VideoTrackFormat.Dim.X; 
	Args.Height    = VideoTrackFormat.Dim.Y;
	Args.Player    = AsShared().ToSharedPtr();
	Args.Timecode = FApp::GetTimecode();

	Args.Time = CurrentTime;
	Args.EvaluationOffsetInSeconds = GetEvaluationOffsetInSeconds();
	Args.Converter = CreateTextureSampleConverter();
	Args.FrameRate = VideoFrameRate;

	check(Args.Converter.IsValid());

	// This sample is going to be used for rendering
	TSharedPtr<FMediaIOCoreTextureSampleBase> NewSample = AcquireTextureSample_AnyThread();
	if (!NewSample.IsValid())
	{
		return nullptr;
	}

	// Setup sample to the converter
	Args.Converter->Setup(NewSample);

	// Initialize the JITR sample
	const bool bInitialized = NewSample->InitializeJITR(Args);

	if (!bInitialized)
	{
		return nullptr;
	}
	NewSample->SetColorConversionSettings(OCIOSettings);

	return NewSample;
}

TSharedPtr<FMediaIOCoreTextureSampleBase> FMediaIOCorePlayerBase::PickSampleToRender_RenderThread(const FFrameInfo& InFrameInformation)
{
	switch (EvaluationType)
	{
	// Pick the latest available sample
	case EMediaIOSampleEvaluationType::Latest:
		return PickSampleToRenderForLatest_RenderThread(InFrameInformation);

	// Pick a sample based on the platform time
	case EMediaIOSampleEvaluationType::PlatformTime:
		return PickSampleToRenderForTimeSynchronized_RenderThread(InFrameInformation);

	// Pick a sample based on the engine's timecode
	case EMediaIOSampleEvaluationType::Timecode:
		return (bFramelock ?
			PickSampleToRenderFramelocked_RenderThread(InFrameInformation) :
			PickSampleToRenderForTimeSynchronized_RenderThread(InFrameInformation));
	default:
		checkNoEntry();
		return PickSampleToRenderForLatest_RenderThread(InFrameInformation);
	}
}

TSharedPtr<FMediaIOCoreTextureSampleBase> FMediaIOCorePlayerBase::PickSampleToRenderForLatest_RenderThread(const FFrameInfo& InFrameInformation)
{
	const TArray<TSharedPtr<IMediaTextureSample>> TextureSamples = Samples->GetVideoSamples();

	// Just return the most recent sample available in the queue
	return TextureSamples.Num() > 0 ?
		StaticCastSharedPtr<FMediaIOCoreTextureSampleBase, IMediaTextureSample, ESPMode::ThreadSafe>(TextureSamples[0]) :
		nullptr;
}

TSharedPtr<FMediaIOCoreTextureSampleBase> FMediaIOCorePlayerBase::PickSampleToRenderForTimeSynchronized_RenderThread(const FFrameInfo& InFrameInformation)
{
	// Reference time based on evaluation type
	FTimespan TargetSampleTimespan;

	// Get base uncorrected reference point
	
	if (EvaluationType == EMediaIOSampleEvaluationType::Timecode && InFrameInformation.RequestedTimecode != FTimecode())
	{
		// We'll use timecode data to find a proper sample
		TargetSampleTimespan = InFrameInformation.RequestedTimecode.ToTimespan(VideoFrameRate);
	}
	else
	{
		// We'll use platform time to find a proper sample
		TargetSampleTimespan = InFrameInformation.SampleTimespan;
	}

	// Apply time correction to the target time
	const FTimespan RequestedOffsetTimespan = FTimespan::FromSeconds(InFrameInformation.EvaluationOffset);
	const FTimespan TargetTimespanCorrected = TargetSampleTimespan - RequestedOffsetTimespan;

	// Go over the sample pool and find a sample closest to the target time
	int32 ClosestIndex = -1;
	int64 SmallestInterval(TNumericLimits<int64>::Max());

	// Get all available video samples
	const TArray<TSharedPtr<IMediaTextureSample>> TextureSamples = Samples->GetVideoSamples();
	if (TextureSamples.Num() == 0)
	{
		return nullptr;
	}
	for (int32 Index = 0; Index < TextureSamples.Num(); ++Index)
	{
		// When EvaluationType == ETimedDataInputEvaluationType::Timecode, the time represents the sample's timecode.
		// When EvaluationType == ETimedDataInputEvaluationType::PlatformTime, the time is based on the platform time.
		const FTimespan TestTimespan = TextureSamples[Index]->GetTime().Time;

		// Either closest positive or closest negative
		const int64 TestInterval = FMath::Abs((TestTimespan - TargetTimespanCorrected).GetTicks());

		// '<=' instead of '<' is used here intentionally. Turns out we might have
		// some samples with the same timecode. To avoid early termination of the search '<=' is used.
		if (TestInterval <= SmallestInterval)
		{
			ClosestIndex = Index;
			SmallestInterval = TestInterval;
		}
		else
		{
			// Since our samples are stored in chronological order, it makes no sense
			// to continue searching. The interval will continue increasing.
			break;
		}
	}

	checkSlow(ClosestIndex >= 0 && ClosestIndex < TextureSamples.Num());
	// Finally, return the closest sample we found
	return StaticCastSharedPtr<FMediaIOCoreTextureSampleBase, IMediaTextureSample, ESPMode::ThreadSafe>(TextureSamples[ClosestIndex]);
}

TSharedPtr<FMediaIOCoreTextureSampleBase> FMediaIOCorePlayerBase::PickSampleToRenderFramelocked_RenderThread(const FFrameInfo& InFrameInformation)
{
	// This isn't an ideal version of frame locking as this will block the render thread of the receiving node to wait for frame from the sending device
	uint32 CurrentLatency = FrameDelay;
	TSharedPtr<FMediaIOCoreTextureSampleBase> SampleToReturn;
	constexpr double TimeoutSeconds = 1.0;
	const double StartTimeSeconds = FPlatformTime::Seconds();
	uint64 FrameNumber = GFrameCounterRenderThread + CurrentLatency - 1;
	while (true)
	{
		const TArray<TSharedPtr<IMediaTextureSample>> TextureSamples = Samples->GetVideoSamples();
		for (int32 Index = 0; Index < TextureSamples.Num(); ++Index)
		{
			SampleToReturn = StaticCastSharedPtr<FMediaIOCoreTextureSampleBase>(TextureSamples[Index]);
			if (SampleToReturn->GetFrameNumber() == FrameNumber)
			{
				return SampleToReturn;
			}
		}
		SampleToReturn = nullptr;
		FPlatformProcess::SleepNoStats(0.001f);
		if ((FPlatformTime::Seconds() - StartTimeSeconds) > TimeoutSeconds)
		{
			break;
		}
	}

	UE_LOGF(LogMediaIOCore, Warning, "Timeout waiting for frame #%llu to be rendered.", FrameNumber);
	// Just return the most recent sample available in the queue
	return nullptr;
}

void FMediaIOCorePlayerBase::TransferTexture_RenderThread(FRHICommandListImmediate& RHICmdList, const TSharedPtr<FMediaIOCoreTextureSampleBase>& Sample, const TSharedPtr<FMediaIOCoreTextureSampleBase>& JITRProxySample)
{
	// DMA P2P transfer
	const bool bIsSampleAwaitingForGpuTransfer = Sample->IsAwaitingForGPUTransfer();
	if (CanUseGPUTextureTransfer() && bIsSampleAwaitingForGpuTransfer)
	{
		RHI_BREADCRUMB_EVENT_STAT(RHICmdList, STAT_MediaIOPlayer_JITR_TransferTexture, "JITR_TransferTexture");

		// Prepare the proxy sample for DMA texture transfer
		PreGPUTransferJITR(Sample, JITRProxySample);

		// Perform fast texture transfer to an external registered texture
		ExecuteGPUTransfer(JITRProxySample);
	}
	// Otherwise set raw data buffer so the texture can be built from it by the caller.
	else
	{
		// Since the proxy sample holds a TSharedPtr reference to this original sample,
		// we can be sure the internal buffer of the source sample won't be released prematurely.
		JITRProxySample->SetBuffer(Sample->GetMutableBuffer());
	}
}

#if WITH_EDITOR
FString FMediaIOCorePlayerBase::GetAnalyticsEventPrefix() const
{
	return TEXT("MediaFramework.MediaIOCorePlayerBase");
}

void FMediaIOCorePlayerBase::RecordSourceOpenedEvent() const
{
	if (FEngineAnalytics::IsAvailable())
	{
		const FString EventName = TEXT("SourceOpened");

		TArray<FAnalyticsEventAttribute> EventAttributes;
		GetAnalyticsEventAttributes(EventName, EventAttributes);

		FEngineAnalytics::GetProvider().RecordEvent(GetAnalyticsEventPrefix() + EventName, EventAttributes);
	}
}

void FMediaIOCorePlayerBase::GetAnalyticsEventAttributes(const FString& EventName, TArray<FAnalyticsEventAttribute>& InOutAttributes) const
{
	InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("ResolutionWidth"), FString::Printf(TEXT("%d"), VideoTrackFormat.Dim.X)));
	InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("ResolutionHeight"), FString::Printf(TEXT("%d"), VideoTrackFormat.Dim.Y)));
	InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("FrameRate"), *VideoFrameRate.ToPrettyText().ToString()));
	InOutAttributes.Add(FAnalyticsEventAttribute(TEXT("UsesOCIO"), OCIOSettings.IsValid() && OCIOSettings->IsValid() ? TEXT("Yes") : TEXT("No")));
}
#endif

void FMediaIOCorePlayerBase::LogBookmark(const FString& Text, const TSharedRef<IMediaTextureSample>& Sample)
{
	// Put a bookmark if requested
	const bool bDetailedInsights = MediaIOCorePlayerDetail::CVarMediaIOJITRInsights.GetValueOnAnyThread();
	if (bDetailedInsights)
	{
		const TOptional<FTimecode> Timecode = Sample->GetTimecode();
		if (Timecode.IsSet())
		{
			const FFrameNumber FrameNumber = Timecode.GetValue().ToFrameNumber(VideoFrameRate);
			TRACE_BOOKMARK(TEXT("%s [%d]"), *Text, FrameNumber.Value % 100);
		}
		else
		{
			const int32 Milliseconds = Sample->GetTime().Time.GetFractionMilli();
			TRACE_BOOKMARK(TEXT("%s [%d]"), *Text, Milliseconds);
		}
	}
}

bool FMediaIOCorePlayerBase::FJITRMediaTextureSamples::FetchVideo(TRange<FTimespan> TimeRange, TSharedPtr<IMediaTextureSample>& OutSample)
{
	if (!ProxySample.IsValid())
	{
		return false;
	}

	OutSample = ProxySample;

	ProxySample.Reset();

	return true;
}

bool FMediaIOCorePlayerBase::FJITRMediaTextureSamples::PeekVideoSampleTime(FMediaTimeStamp& TimeStamp)
{
	return false;
}

void FMediaIOCorePlayerBase::FJITRMediaTextureSamples::FlushSamples()
{
	ProxySample.Reset();
}


#undef LOCTEXT_NAMESPACE
