// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaIOCoreSamples.h"

#include "IMediaAudioSample.h"
#include "IMediaBinarySample.h"
#include "IMediaOverlaySample.h"
#include "IMediaTextureSample.h"


FMediaIOCoreSamples::FMediaIOCoreSamples()
	: VideoSamples("Video")
	, AudioSamples("Audio")
	, MetadataSamples("Metadata")
	, SubtitleSamples("Subtitles")
	, CaptionSamples("Caption")
{

}

void FMediaIOCoreSamples::CacheSamplesState(FTimespan PlayerTime)
{
	VideoSamples.CacheState(PlayerTime);
	AudioSamples.CacheState(PlayerTime);
	MetadataSamples.CacheState(PlayerTime);
	SubtitleSamples.CacheState(PlayerTime);
	CaptionSamples.CacheState(PlayerTime);
}

void FMediaIOCoreSamples::EnableTimedDataChannels(ITimedDataInput* Input, EMediaIOSampleType SampleTypes)
{
	VideoSamples.EnableChannel(Input,  EnumHasAnyFlags(SampleTypes, EMediaIOSampleType::Video));
	AudioSamples.EnableChannel(Input, EnumHasAnyFlags(SampleTypes, EMediaIOSampleType::Audio));
	MetadataSamples.EnableChannel(Input, EnumHasAnyFlags(SampleTypes, EMediaIOSampleType::Metadata));
	SubtitleSamples.EnableChannel(Input, EnumHasAnyFlags(SampleTypes, EMediaIOSampleType::Subtitles));
	CaptionSamples.EnableChannel(Input, EnumHasAnyFlags(SampleTypes, EMediaIOSampleType::Caption));
}

void FMediaIOCoreSamples::InitializeVideoBuffer(const FMediaIOSamplingSettings& InSettings)
{
	VideoSamples.UpdateSettings(InSettings);
}

void FMediaIOCoreSamples::InitializeAudioBuffer(const FMediaIOSamplingSettings& InSettings)
{
	AudioSamples.UpdateSettings(InSettings);
}

void FMediaIOCoreSamples::InitializeMetadataBuffer(const FMediaIOSamplingSettings& InSettings)
{
	MetadataSamples.UpdateSettings(InSettings);
}

void FMediaIOCoreSamples::InitializeSubtitlesBuffer(const FMediaIOSamplingSettings& InSettings)
{
	SubtitleSamples.UpdateSettings(InSettings);
}

void FMediaIOCoreSamples::InitializeCaptionBuffer(const FMediaIOSamplingSettings& InSettings)
{
	CaptionSamples.UpdateSettings(InSettings);
}


/* IMediaSamples interface
*****************************************************************************/

bool FMediaIOCoreSamples::FetchAudio(TRange<FTimespan> TimeRange, TSharedPtr<IMediaAudioSample, ESPMode::ThreadSafe>& OutSample)
{
	return AudioSamples.FetchSample(TimeRange, OutSample);
}


bool FMediaIOCoreSamples::FetchCaption(TRange<FTimespan> TimeRange, TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>& OutSample)
{
	return CaptionSamples.FetchSample(TimeRange, OutSample);
}


bool FMediaIOCoreSamples::FetchMetadata(TRange<FTimespan> TimeRange, TSharedPtr<IMediaBinarySample, ESPMode::ThreadSafe>& OutSample)
{
	return MetadataSamples.FetchSample(TimeRange, OutSample);
}


bool FMediaIOCoreSamples::FetchSubtitle(TRange<FTimespan> TimeRange, TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>& OutSample)
{
	return SubtitleSamples.FetchSample(TimeRange, OutSample);
}


bool FMediaIOCoreSamples::FetchVideo(TRange<FTimespan> TimeRange, TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& OutSample)
{
	return VideoSamples.FetchSample(TimeRange, OutSample);
}


void FMediaIOCoreSamples::FlushSamples()
{
	AudioSamples.FlushSamples();
	CaptionSamples.FlushSamples();
	MetadataSamples.FlushSamples();
	SubtitleSamples.FlushSamples();
	VideoSamples.FlushSamples();
}

bool FMediaIOCoreSamples::PeekVideoSampleTime(FMediaTimeStamp & TimeStamp)
{
	// player does not support v2 timing control at this point -> no need for this method, yet
	return false;
}

void FMediaIOCoreSamples::SetSampleBufferSize(int32 BufferSize)
{
	AudioSamples.SetDataBufferSize(BufferSize);
	CaptionSamples.SetDataBufferSize(BufferSize);
	MetadataSamples.SetDataBufferSize(BufferSize);
	SubtitleSamples.SetDataBufferSize(BufferSize);
	VideoSamples.SetDataBufferSize(BufferSize);
}
