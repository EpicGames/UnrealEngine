// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderTargetMediaCapture.h"

#include "Engine/Engine.h"
#include "Engine/TextureRenderTarget2D.h"
#include "MediaIOCoreModule.h"
#include "MediaTexture.h"
#include "RenderTargetMediaOutput.h"
#include "Slate/SceneViewport.h"
#include "TextureResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RenderTargetMediaCapture)


void URenderTargetMediaCapture::OnRHIResourceCaptured_RenderingThread(
	FRHICommandListImmediate& RHICmdList, 
	const FCaptureBaseData& InBaseData, 
	TSharedPtr<FMediaCaptureUserData, 
	ESPMode::ThreadSafe> InUserData, 
	FTextureRHIRef InTexture
)
{
	FScopeLock ScopeLock(&RenderTargetCriticalSection);

	if (!RenderTarget || !RenderTarget->GetRenderTargetResource())
	{
		return;
	}
	
	const FTextureRHIRef RenderTargetRHIRef = RenderTarget->GetRenderTargetResource()->GetRenderTargetTexture();

	if (!RenderTargetRHIRef.IsValid())
	{
		return;
	}

	// copy the captured texture into our render target

	RHICmdList.CopyTexture(
		InTexture,
		RenderTargetRHIRef,
		FRHICopyTextureInfo()
	);

	RenderTarget->GetRenderTargetResource()->FlushDeferredResourceUpdate(RHICmdList);
}

bool URenderTargetMediaCapture::InitializeCapture()
{
	return true;
}

bool URenderTargetMediaCapture::PostInitializeCaptureViewport(TSharedPtr<FSceneViewport>& InSceneViewport)
{
	const FTextureRHIRef& BackBuffer = InSceneViewport->GetRenderTargetTexture();

	if (!BackBuffer.IsValid())
	{
		UE_LOGF(LogMediaIOCore, Error, "RenderTargetMediaCapture: Scene viewport had no backbuffer.");
		return false;
	}

	const FRHITextureDesc& Desc = BackBuffer->GetDesc();
	return StartNewCapture(Desc.Extent, Desc.Format);
}

bool URenderTargetMediaCapture::PostInitializeCaptureRenderTarget(UTextureRenderTarget2D* InRenderTarget)
{
	return StartNewCapture(FIntPoint(InRenderTarget->SizeX, InRenderTarget->SizeY), InRenderTarget->GetFormat());
}

bool URenderTargetMediaCapture::PostInitializeCaptureRHIResource(const FRHICaptureResourceDescription& InResourceDescription)
{
	return StartNewCapture(InResourceDescription.ResourceSize, InResourceDescription.PixelFormat);
}

bool URenderTargetMediaCapture::PostInitializeCaptureMediaTexture(UMediaTexture* InMediaTexture)
{
	return StartNewCapture(FIntPoint(InMediaTexture->GetWidth(), InMediaTexture->GetHeight()), EPixelFormat::PF_B8G8R8A8);
}

void URenderTargetMediaCapture::StopCaptureImpl(bool /*bAllowPendingFrameToBeProcess*/)
{
	TRACE_BOOKMARK(TEXT("RenderTargetMediaCapture::StopCapture"));
}

bool URenderTargetMediaCapture::StartNewCapture(const FIntPoint& InSourceTargetSize, EPixelFormat InSourceTargetFormat)
{
	TRACE_BOOKMARK(TEXT("RenderTargetMediaCapture::StartNewCapture"));
	{
		RenderTarget = nullptr;
		
		const URenderTargetMediaOutput* RenderTargetMediaOutput = CastChecked<URenderTargetMediaOutput>(MediaOutput);

		if (RenderTargetMediaOutput)
		{
			FScopeLock ScopeLock(&RenderTargetCriticalSection);

			RenderTarget = RenderTargetMediaOutput->RenderTarget.LoadSynchronous();

			if (RenderTarget)
			{
				SetState(EMediaCaptureState::Capturing);
				return true;
			}
			else
			{
				UE_LOGF(LogMediaIOCore, Error, "RenderTargetMediaCapture: Missing render target object: '%ls'.", *RenderTargetMediaOutput->RenderTarget.ToString());
			}
		}
		else
		{
			UE_LOGF(LogMediaIOCore, Error, "RenderTargetMediaCapture: Media Output not found or of unexpected type.");
		}
	}

	SetState(EMediaCaptureState::Error);

	return false;
}
