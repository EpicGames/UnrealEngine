// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderTargetMediaOutput.h"

#include "Engine/TextureRenderTarget2D.h"
#include "MediaIOCoreModule.h"
#include "RenderTargetMediaCapture.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RenderTargetMediaOutput)

bool URenderTargetMediaOutput::Validate(FString& OutFailureReason) const
{
	if (!Super::Validate(OutFailureReason))
	{
		return false;
	}

	const UTextureRenderTarget2D* LoadedRenderTarget = RenderTarget.LoadSynchronous();

	if (!LoadedRenderTarget)
	{
		OutFailureReason = FString::Printf(TEXT("Missing render target object: '%s'."), *RenderTarget.ToString());
		return false;
	}

	return true;
}

FIntPoint URenderTargetMediaOutput::GetRequestedSize() const
{
	if (const UTextureRenderTarget2D* LoadedRenderTarget = RenderTarget.LoadSynchronous())
	{
		return FIntPoint(LoadedRenderTarget->SizeX, LoadedRenderTarget->SizeY);
	}

	return RequestCaptureSourceSize;
}


EPixelFormat URenderTargetMediaOutput::GetRequestedPixelFormat() const
{
	if (const UTextureRenderTarget2D* LoadedRenderTarget = RenderTarget.LoadSynchronous())
	{
		return LoadedRenderTarget->GetFormat();
	}

	return EPixelFormat::PF_Unknown;
}

EMediaCaptureConversionOperation URenderTargetMediaOutput::GetConversionOperation(EMediaCaptureSourceType /*InSourceType*/) const
{
	if (bInvertAlpha)
	{
		return EMediaCaptureConversionOperation::INVERT_ALPHA;
	}

	return EMediaCaptureConversionOperation::NONE;
}

UMediaCapture* URenderTargetMediaOutput::CreateMediaCaptureImpl()
{
	UMediaCapture* Result = NewObject<URenderTargetMediaCapture>();

	if (Result)
	{
		UE_LOGF(LogMediaIOCore, Log, "Created Render Target Media Capture");
		Result->SetMediaOutput(this);
	}

	return Result;
}
