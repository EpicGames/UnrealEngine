// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaOutput.h"
#include "UObject/SoftObjectPtr.h"
#include "RenderTargetMediaOutput.generated.h"

class UTextureRenderTarget2D;

/**
 * Captures Media to a Render Target. This can be useful to make a texture available
 * from a source that has a Media output interface.
 */
UCLASS(BlueprintType)
class URenderTargetMediaOutput : public UMediaOutput
{
	GENERATED_BODY()

public:

	/** Invert the alpha channel */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Media")
	bool bInvertAlpha = false;

	/** Specify the render target to be capturing to. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Media")
	TSoftObjectPtr<UTextureRenderTarget2D> RenderTarget;
	
	//~ Begin UMediaOutput interface

public:

	virtual bool Validate(FString& FailureReason) const override;
	virtual FIntPoint GetRequestedSize() const override;
	virtual EPixelFormat GetRequestedPixelFormat() const override;
	virtual EMediaCaptureConversionOperation GetConversionOperation(EMediaCaptureSourceType InSourceType) const override;

protected:

	virtual UMediaCapture* CreateMediaCaptureImpl() override;

	//~ End UMediaOutput interface
};
