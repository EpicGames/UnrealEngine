// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaIOCoreAudioSampleBase.h"


FMediaIOCoreAudioSampleBase::FMediaIOCoreAudioSampleBase()
	: Channels(0)
	, Duration(0)
	, SampleRate(0)
	, Time(FTimespan::MinValue())
{ }


bool FMediaIOCoreAudioSampleBase::Initialize(const int32* InAudioBuffer, uint32 InBufferSize, uint32 InNumberOfChannels, uint32 InSampleRate, FTimespan InTime, const TOptional<FTimecode>& InTimecode)
{
	FreeSample();

	if (!SetProperties(InBufferSize, InNumberOfChannels, InSampleRate, InTime, InTimecode))
	{
		return false;
	}

	return SetBuffer(InAudioBuffer, InBufferSize);
}


bool FMediaIOCoreAudioSampleBase::Initialize(TArray<int32> InAudioBuffer, uint32 InNumberOfChannels, uint32 InSampleRate, FTimespan InTime, const TOptional<FTimecode>& InTimecode)
{
	FreeSample();

	if (!SetProperties(InAudioBuffer.Num(), InNumberOfChannels, InSampleRate, InTime, InTimecode))
	{
		return false;
	}

	return SetBuffer(MoveTemp(InAudioBuffer));
}


bool FMediaIOCoreAudioSampleBase::SetBuffer(const int32* InAudioBuffer, uint32 InBufferSize)
{
	if (InAudioBuffer == nullptr)
	{
		return false;
	}

	Buffer.Reset(InBufferSize);
	Buffer.Append(InAudioBuffer, InBufferSize);

	return true;
}


bool FMediaIOCoreAudioSampleBase::SetBuffer(TArray<int32> InAudioBuffer)
{
	if (InAudioBuffer.Num() == 0)
	{
		return false;
	}

	Buffer = MoveTemp(InAudioBuffer);

	return true;
}

bool FMediaIOCoreAudioSampleBase::SetProperties(uint32 InBufferSize, uint32 InNumberOfChannels, uint32 InSampleRate, FTimespan InTime, const TOptional<FTimecode>& InTimecode)
{
	if (InNumberOfChannels * InSampleRate <= 0)
	{
		return false;
	}

	Time = InTime;
	Timecode = InTimecode;
	Channels = InNumberOfChannels;
	SampleRate = InSampleRate;
	Duration = (InBufferSize * ETimespan::TicksPerSecond) / (Channels * SampleRate);

	Buffer.SetNum(InBufferSize, EAllowShrinking::No);

	return true;
}


void* FMediaIOCoreAudioSampleBase::RequestBuffer(uint32 InBufferSize)
{
	FreeSample();
	Buffer.SetNumUninitialized(InBufferSize); // Reset the array without shrinking (Does not destruct items, does not de-allocate memory).
	return Buffer.GetData();
}
