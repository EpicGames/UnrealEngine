// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaIOCoreBinarySampleBase.h"


FMediaIOCoreBinarySampleBase::FMediaIOCoreBinarySampleBase()
	: Time(FTimespan::Zero())
{ }


bool FMediaIOCoreBinarySampleBase::Initialize(const uint8* InBinaryBuffer, uint32 InBufferSize, FTimespan InTime, const FFrameRate& InFrameRate, const TOptional<FTimecode>& InTimecode)
{
	FreeSample();

	if (!SetProperties(InTime, InFrameRate, InTimecode))
	{
		return false;
	}

	return SetBuffer(InBinaryBuffer, InBufferSize);
}


bool FMediaIOCoreBinarySampleBase::Initialize(TArray<uint8> InBinaryBuffer, FTimespan InTime, const FFrameRate& InFrameRate, const TOptional<FTimecode>& InTimecode)
{
	FreeSample();

	if (!SetProperties(InTime, InFrameRate, InTimecode))
	{
		return false;
	}

	return SetBuffer(InBinaryBuffer);
}


bool FMediaIOCoreBinarySampleBase::SetBuffer(const uint8* InBinaryBuffer, uint32 InBufferSize)
{
	if (InBinaryBuffer == nullptr)
	{
		return false;
	}

	Buffer.Reset(InBufferSize);
	Buffer.Append(InBinaryBuffer, InBufferSize);

	return true;
}


bool FMediaIOCoreBinarySampleBase::SetBuffer(TArray<uint8> InBinaryBuffer)
{
	if (InBinaryBuffer.Num() == 0)
	{
		return false;
	}

	Buffer = MoveTemp(InBinaryBuffer);

	return true;
}


bool FMediaIOCoreBinarySampleBase::SetProperties(FTimespan InTime, const FFrameRate& InFrameRate, const TOptional<FTimecode>& InTimecode)
{
	Time = InTime;
	Duration = FTimespan(ETimespan::TicksPerSecond * InFrameRate.AsInterval());
	Timecode = InTimecode;
	return true;
}


void* FMediaIOCoreBinarySampleBase::RequestBuffer(uint32 InBufferSize)
{
	FreeSample();
	Buffer.SetNumUninitialized(InBufferSize); // Reset the array without shrinking (Does not destruct items, does not de-allocate memory).
	return Buffer.GetData();
}
