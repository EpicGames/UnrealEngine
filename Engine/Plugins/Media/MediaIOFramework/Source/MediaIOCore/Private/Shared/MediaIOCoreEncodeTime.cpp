// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaIOCoreEncodeTime.h"

FMediaIOCoreEncodeTime::FMediaIOCoreEncodeTime(EMediaIOCoreEncodePixelFormat InFormat, void* InBuffer, uint32 InPitch, uint32 InWidth, uint32 InHeight, uint32 XOffset, uint32 YOffset)
	: Format(InFormat)
	, Buffer(InBuffer)
	, Pitch(InPitch)
	, Width(InWidth)
	, Height(InHeight)
	, XPixelOffset(XOffset)
	, YPixelOffset(YOffset)
{
	check(Buffer);

	if (Format == EMediaIOCoreEncodePixelFormat::A2B10G10R10)
	{
		// 10bit Encoding of colors (MSB = A, LSB = R)
		ColorBlack = 0xC0000000;
		ColorWhite = 0xFFFFFFFF;
	}
	else if (Format == EMediaIOCoreEncodePixelFormat::CharUYVY)
	{
		// Handled directly in SetPixel()
	}
	else if (Format == EMediaIOCoreEncodePixelFormat::CharBGRA)
	{
		ColorBlack = FColor::Black.ToPackedARGB();
		ColorWhite = FColor::White.ToPackedARGB();
	}
	else if (Format == EMediaIOCoreEncodePixelFormat::YUVv210)
	{
		// Handled directly in SetPixel()
	}
	else
	{
		check(false);
	}
}

// Monochrome version of Unreal Engine Small Font, 8x11 bitmap per character
// Contains: 0123456789:

const uint32 MaxCharacter = 11;
const uint32 CharacterHeight = 11;
const uint8 Font[MaxCharacter][CharacterHeight] =
{
	{0x00, 0x3C, 0x42, 0x42, 0x42, 0x42, 0x42, 0x42, 0x42, 0x3C, 0x00}, // 0
	{0x00, 0x08, 0x0E, 0x08, 0x08, 0x08, 0x08, 0x08, 0x08, 0x3E, 0x00}, // 1
	{0x00, 0x3C, 0x42, 0x42, 0x40, 0x20, 0x18, 0x04, 0x02, 0x7E, 0x00}, // 2
	{0x00, 0x3C, 0x42, 0x40, 0x40, 0x38, 0x40, 0x40, 0x42, 0x3C, 0x00}, // 3
	{0x00, 0x20, 0x30, 0x28, 0x24, 0x22, 0x7E, 0x20, 0x20, 0x20, 0x00}, // 4
	{0x00, 0x7C, 0x04, 0x04, 0x04, 0x3C, 0x40, 0x40, 0x42, 0x3C, 0x00}, // 5
	{0x00, 0x38, 0x04, 0x02, 0x3E, 0x42, 0x42, 0x42, 0x42, 0x3C, 0x00}, // 6
	{0x00, 0x7E, 0x40, 0x20, 0x20, 0x10, 0x10, 0x08, 0x08, 0x08, 0x00}, // 7
	{0x00, 0x3C, 0x42, 0x42, 0x42, 0x3C, 0x42, 0x42, 0x42, 0x3C, 0x00}, // 8
	{0x00, 0x3C, 0x42, 0x42, 0x42, 0x42, 0x7C, 0x40, 0x20, 0x1C, 0x00}, // 9
	{0x00, 0x00, 0x08, 0x08, 0x00, 0x00, 0x00, 0x08, 0x08, 0x00, 0x00}, // :
};

void FMediaIOCoreEncodeTime::SetPixelScaled(uint32 InX, uint32 InY, bool InSet, uint32 InScale) const
{
	for (uint32 ScaleY = 0; ScaleY < InScale; ScaleY++)
	{
		for (uint32 ScaleX = 0; ScaleX < InScale; ScaleX++)
		{
			SetPixel((InX+XPixelOffset) * InScale + ScaleX, (InY+YPixelOffset) * InScale + ScaleY, InSet);
		}
	}
}

void FMediaIOCoreEncodeTime::SetPixel(uint32 InX, uint32 InY, bool InSet) const
{
	if ((InX >= Width) || (InY >= Height))
	{
		return;
	}

	if (Format == EMediaIOCoreEncodePixelFormat::YUVv210)
	{
		// Every 6 pixels is encoded in a 128bits block
		uint32 Block = InX / 6;
		uint32 Pixel = InX % 6;
		const uint32 YBlockIndex[6] = { 0, 1, 1, 2, 3, 3 };
		const uint32 YBlockOffset[6] = { 10, 0, 20, 10, 0, 20 };
		const uint32 UBlockIndex[6] = { 0, 0, 1, 1, 2, 2 };
		const uint32 UBlockOffset[6] = { 0, 0, 10, 10, 20, 20 };
		const uint32 VBlockIndex[6] = { 0, 0, 2, 2, 3, 3 };
		const uint32 VBlockOffset[6] = { 20, 20, 0, 0, 10, 10 };

		uint32* BlockPointer = (reinterpret_cast<uint32*>((reinterpret_cast<char*>(Buffer) + (Pitch * InY)) + Block * 16));
		if (InSet)
		{
			// White
			BlockPointer[YBlockIndex[Pixel]] |= 0x3ff << YBlockOffset[Pixel];
		}
		else 
		{
			// Black
			BlockPointer[YBlockIndex[Pixel]] &= (~(0x3ff << YBlockOffset[Pixel]));
		}

		// Always clear Chroma
		BlockPointer[UBlockIndex[Pixel]] &= (~(0x3ff << UBlockOffset[Pixel]));
		BlockPointer[UBlockIndex[Pixel]] |= 0x1ff << UBlockOffset[Pixel];
		BlockPointer[VBlockIndex[Pixel]] &= (~(0x3ff << VBlockOffset[Pixel]));
		BlockPointer[VBlockIndex[Pixel]] |= 0x1ff << VBlockOffset[Pixel];

	}
	else if (Format == EMediaIOCoreEncodePixelFormat::CharUYVY)
	{
		uint32 Block = InX / 2;
		uint32 Pixel = InX % 2;
		uint32* BlockPointer = (reinterpret_cast<uint32*>((reinterpret_cast<char*>(Buffer) + (Pitch * InY)) + Block * 4));
		
		if (InSet)
		{
			// White
			if (Pixel == 0)
			{
				BlockPointer[0] |= 0xff000000;
			}
			else
			{
				BlockPointer[0] |= 0x0000ff00;
			}
		}
		else
		{
			// Black
			if (Pixel == 0)
			{
				BlockPointer[0] &= ~0xff000000;
			}
			else
			{
				BlockPointer[0] &= ~0x0000ff00;
			}
		}

		// Always clear Chroma
		BlockPointer[0] &= 0xff00ff00;
		BlockPointer[0] |= 0x00800080;
	}
	else
	{
		if (InSet)
		{
			At(InX, InY) = ColorWhite;
		}
		else
		{
			At(InX, InY) = ColorBlack;
		}
	}
}


void FMediaIOCoreEncodeTime::DrawChar(uint32 InX, uint32 InChar) const
{
	if (InChar >= MaxCharacter)
	{
		return;
	}

	for (int y = 0; y < MaxCharacter; y++)
	{
		for (int x = 0; x < 8; x++)
		{
			if (Font[InChar][y] & (1 << x))
			{
				SetPixelScaled(InX * 8 + x, y, true, 4);
			}
			else
			{
				SetPixelScaled(InX * 8 + x, y, false, 4);
			}
		}
	}
}


void FMediaIOCoreEncodeTime::DrawTime(uint32 InX, uint32 InTime) const
{
	uint32 Hi = (InTime / 10) % 10;
	uint32 Lo = InTime % 10;

	DrawChar(InX, Hi);
	DrawChar(InX + 1, Lo);
}


void FMediaIOCoreEncodeTime::Render(uint32 InHours, uint32 InMinutes, uint32 InSeconds, uint32 InFrames) const
{
	const uint32 ColonCharacterIndex = 10;
	DrawTime(0, InHours);
	DrawChar(2, ColonCharacterIndex);
	DrawTime(3, InMinutes);
	DrawChar(5, ColonCharacterIndex);
	DrawTime(6, InSeconds);
	DrawChar(8, ColonCharacterIndex);
	DrawTime(9, InFrames);
}
