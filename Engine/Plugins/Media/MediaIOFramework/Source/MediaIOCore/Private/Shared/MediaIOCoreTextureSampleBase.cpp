// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaIOCoreTextureSampleBase.h"

#include "Async/Async.h"
#include "ColorManagement/TransferFunctions.h"
#include "MediaIOCoreTextureSampleConverter.h"

#include "OpenColorIOConfiguration.h"
#include "OpenColorIOColorSpace.h"
#include "OpenColorIORendering.h"
#include "OpenColorIOShader.h"
#include "OpenColorIOShared.h"
#include "RenderGraphBuilder.h"
#include "RHI.h"
#include "RHIResources.h"
#include "ScreenPass.h"
DECLARE_GPU_STAT(MediaIO_ColorConversion);

FMediaIOCoreTextureSampleBase::FMediaIOCoreTextureSampleBase()
	: Duration(FTimespan::Zero())
	, SampleFormat(EMediaTextureSampleFormat::Undefined)
	, Time(FTimespan::Zero())
	, FrameNumber(0)
	, Stride(0)
	, Width(0)
	, Height(0)
	, bIsAwaitingForGPUTransfer(false)
	, SourceColorSettings()
{
}


bool FMediaIOCoreTextureSampleBase::Initialize(const void* InVideoBuffer, uint32 InBufferSize, uint32 InStride, uint32 InWidth, uint32 InHeight, EMediaTextureSampleFormat InSampleFormat, FTimespan InTime, const FFrameRate& InFrameRate, const TOptional<FTimecode>& InTimecode, TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> InSourceColorSettings)
{
	FreeSample();

	if (!SetProperties(InStride, InWidth, InHeight, InSampleFormat, InTime, InFrameRate, InTimecode, InSourceColorSettings))
	{
		return false;
	}

	return SetBuffer(InVideoBuffer, InBufferSize);
}


bool FMediaIOCoreTextureSampleBase::Initialize(const TArray<uint8>& InVideoBuffer, uint32 InStride, uint32 InWidth, uint32 InHeight, EMediaTextureSampleFormat InSampleFormat, FTimespan InTime, const FFrameRate& InFrameRate, const TOptional<FTimecode>& InTimecode, TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> InSourceColorSettings)
{
	FreeSample();

	if (!SetProperties(InStride, InWidth, InHeight, InSampleFormat, InTime, InFrameRate, InTimecode, InSourceColorSettings))
	{
		return false;
	}

	return SetBuffer(InVideoBuffer);
}


bool FMediaIOCoreTextureSampleBase::Initialize(TArray<uint8>&& InVideoBuffer, uint32 InStride, uint32 InWidth, uint32 InHeight, EMediaTextureSampleFormat InSampleFormat, FTimespan InTime, const FFrameRate& InFrameRate, const TOptional<FTimecode>& InTimecode, TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> InSourceColorSettings)
{
	FreeSample();

	if (!SetProperties(InStride, InWidth, InHeight, InSampleFormat, InTime, InFrameRate, InTimecode, InSourceColorSettings))
	{
		return false;
	}

	return SetBuffer(MoveTemp(InVideoBuffer));
}


bool FMediaIOCoreTextureSampleBase::SetBuffer(const void* InVideoBuffer, uint32 InBufferSize)
{
	if (InVideoBuffer == nullptr)
	{
		return false;
	}

	Buffer.Reset(InBufferSize);
	Buffer.Append(reinterpret_cast<const uint8*>(InVideoBuffer), InBufferSize);

	return true;
}


bool FMediaIOCoreTextureSampleBase::SetBuffer(const TArray<uint8>& InVideoBuffer)
{
	if (InVideoBuffer.Num() == 0)
	{
		return false;
	}

	Buffer = InVideoBuffer;

	return true;
}

bool FMediaIOCoreTextureSampleBase::SetBuffer(TArray<uint8>&& InVideoBuffer)
{
	if (InVideoBuffer.Num() == 0)
	{
		return false;
	}

	Buffer = MoveTemp(InVideoBuffer);

	return true;
}

bool FMediaIOCoreTextureSampleBase::SetProperties(uint32 InStride, uint32 InWidth, uint32 InHeight, EMediaTextureSampleFormat InSampleFormat, FTimespan InTime, const FFrameRate& InFrameRate, const TOptional<FTimecode>& InTimecode, TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> InSourceColorSettings)
{
	if (InSampleFormat == EMediaTextureSampleFormat::Undefined)
	{
		return false;
	}

	Stride = InStride;
	Width = InWidth;
	Height = InHeight;
	SampleFormat = InSampleFormat;
	Time = InTime;
	Duration = FTimespan(ETimespan::TicksPerSecond * InFrameRate.AsInterval());
	Timecode = InTimecode;
	SourceColorSettings = InSourceColorSettings;

	return true;
}

bool FMediaIOCoreTextureSampleBase::InitializeWithEvenOddLine(bool bUseEvenLine, const void* InVideoBuffer, uint32 InBufferSize, uint32 InStride, uint32 InWidth, uint32 InHeight, EMediaTextureSampleFormat InSampleFormat, FTimespan InTime, const FFrameRate& InFrameRate, const TOptional<FTimecode>& InTimecode, TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> InSourceColorSettings)
{
	FreeSample();

	if (!SetProperties(InStride, InWidth, InHeight/2, InSampleFormat, InTime, InFrameRate, InTimecode, InSourceColorSettings))
	{
		return false;
	}

	return SetBufferWithEvenOddLine(bUseEvenLine, InVideoBuffer, InBufferSize, InStride, InHeight);
}

bool FMediaIOCoreTextureSampleBase::SetBufferWithEvenOddLine(bool bUseEvenLine, const void* InVideoBuffer, uint32 InBufferSize, uint32 InStride, uint32 InHeight)
{
	Buffer.Reset(InBufferSize / 2);

	for (uint32 IndexY = (bUseEvenLine ? 0 : 1); IndexY < InHeight; IndexY += 2)
	{
		const uint8* Source = reinterpret_cast<const uint8*>(InVideoBuffer) + (IndexY*InStride);
		Buffer.Append(Source, InStride);
	}

	return true;
}

void FMediaIOCoreTextureSampleBase::SetColorConversionSettings(TSharedPtr<struct FOpenColorIOColorConversionSettings> InColorConversionSettings)
{
	ColorConversionSettings = InColorConversionSettings;
	if (IsInGameThread())
	{
		CacheColorCoversionSettings_GameThread();
	}
	else
	{
		AsyncTask(ENamedThreads::GameThread, [WeakThis = AsWeak()]()
			{
				TSharedPtr<FMediaIOCoreTextureSampleBase> This = WeakThis.Pin();
				if (This.IsValid())
				{
					This->CacheColorCoversionSettings_GameThread();
				}
			}
		);
	}
}

void* FMediaIOCoreTextureSampleBase::RequestBuffer(uint32 InBufferSize)
{
	FreeSample();
	Buffer.SetNumUninitialized(InBufferSize); // Reset the array without shrinking (Does not destruct items, does not de-allocate memory).
	return Buffer.GetData();
}

void* FMediaIOCoreTextureSampleBase::GetOrRequestBuffer(uint32 InBufferSize)
{
	if (Buffer.Num() != InBufferSize)
	{
		RequestBuffer(InBufferSize);
	}

	return Buffer.GetData();
}

TSharedPtr<FMediaIOCorePlayerBase> FMediaIOCoreTextureSampleBase::GetPlayer() const
{
	return Player.Pin();
}

bool FMediaIOCoreTextureSampleBase::InitializeJITR(const FMediaIOCoreSampleJITRConfigurationArgs& Args)
{
	if (!Args.Player|| !Args.Converter)
	{
		return false;
	}

	// Native sample data
	Width  = Args.Width;
	Height = Args.Height;
	Time   = Args.Time;
	Timecode = Args.Timecode;
	FrameNumber = GFrameNumber;
	Duration = FTimespan(ETimespan::TicksPerSecond * Args.FrameRate.AsInterval());

	// JITR data
	Player    = Args.Player;
	Converter = Args.Converter;
	EvaluationOffsetInSeconds = Args.EvaluationOffsetInSeconds;
	return true;
}

void FMediaIOCoreTextureSampleBase::CopyConfiguration(const TSharedPtr<FMediaIOCoreTextureSampleBase>& SourceSample)
{
	if (!SourceSample.IsValid())
	{
		return;
	}

	// Copy configuration parameters
	Stride = SourceSample->Stride;
	Width  = SourceSample->Width;
	Height = SourceSample->Height;
	SampleFormat = SourceSample->SampleFormat;
	Time = SourceSample->Time;
	Timecode = SourceSample->Timecode;
	SourceColorSettings = SourceSample->SourceColorSettings;
	ColorConversionSettings = SourceSample->ColorConversionSettings;
	CachedOCIOResources = SourceSample->CachedOCIOResources;
	Player = SourceSample->Player;
	Converter = SourceSample->Converter;
	FrameNumber = SourceSample->FrameNumber.load();
	Duration = SourceSample->Duration;
	Texture = SourceSample->Texture;

	EvaluationOffsetInSeconds = SourceSample->EvaluationOffsetInSeconds;

	// Save original sample
	OriginalSample = SourceSample;
}

// Deprecated methods
PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool FMediaIOCoreTextureSampleBase::Initialize(const void* InVideoBuffer, uint32 InBufferSize, uint32 InStride, uint32 InWidth, uint32 InHeight, EMediaTextureSampleFormat InSampleFormat, FTimespan InTime, const FFrameRate& InFrameRate, const TOptional<FTimecode>& InTimecode, const UE::MediaIOCore::FColorFormatArgs& InColorFormatArgs)
{
	return Initialize(InVideoBuffer, InBufferSize, InStride, InWidth, InHeight, InSampleFormat, InTime, InFrameRate, InTimecode, nullptr);
}

bool FMediaIOCoreTextureSampleBase::Initialize(const TArray<uint8>& InVideoBuffer, uint32 InStride, uint32 InWidth, uint32 InHeight, EMediaTextureSampleFormat InSampleFormat, FTimespan InTime, const FFrameRate& InFrameRate, const TOptional<FTimecode>& InTimecode, const UE::MediaIOCore::FColorFormatArgs& InColorFormatArgs)
{
	return Initialize(InVideoBuffer, InStride, InWidth, InHeight, InSampleFormat, InTime, InFrameRate, InTimecode, nullptr);
}

bool FMediaIOCoreTextureSampleBase::Initialize(TArray<uint8>&& InVideoBuffer, uint32 InStride, uint32 InWidth, uint32 InHeight, EMediaTextureSampleFormat InSampleFormat, FTimespan InTime, const FFrameRate& InFrameRate, const TOptional<FTimecode>& InTimecode, const UE::MediaIOCore::FColorFormatArgs& InColorFormatArgs)
{
	return Initialize(InVideoBuffer, InStride, InWidth, InHeight, InSampleFormat, InTime, InFrameRate, InTimecode, nullptr);
}

bool FMediaIOCoreTextureSampleBase::SetProperties(uint32 InStride, uint32 InWidth, uint32 InHeight, EMediaTextureSampleFormat InSampleFormat, FTimespan InTime, const FFrameRate& InFrameRate, const TOptional<FTimecode>& InTimecode, const UE::MediaIOCore::FColorFormatArgs& InColorFormatArgs)
{
	return SetProperties(InStride, InWidth, InHeight, InSampleFormat, InTime, InFrameRate, InTimecode, nullptr);
}

bool FMediaIOCoreTextureSampleBase::InitializeWithEvenOddLine(bool bUseEvenLine, const void* InVideoBuffer, uint32 InBufferSize, uint32 InStride, uint32 InWidth, uint32 InHeight, EMediaTextureSampleFormat InSampleFormat, FTimespan InTime, const FFrameRate& InFrameRate, const TOptional<FTimecode>& InTimecode, const UE::MediaIOCore::FColorFormatArgs& InColorFormatArgs)
{
	return InitializeWithEvenOddLine(bUseEvenLine, InVideoBuffer, InBufferSize, InStride, InWidth, InHeight, InSampleFormat, InTime, InFrameRate, InTimecode, nullptr);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FMediaIOCoreTextureSampleBase::CacheColorCoversionSettings_GameThread()
{
	if (ColorConversionSettings.IsValid() && ColorConversionSettings->IsValid())
	{
		CachedOCIOResources = MakeShared<FOpenColorIORenderPassResources>();

		FOpenColorIORenderPassResources Resources = FOpenColorIORendering::GetRenderPassResources(*ColorConversionSettings, GMaxRHIFeatureLevel);
		CachedOCIOResources->ShaderResource = Resources.ShaderResource;
		CachedOCIOResources->TextureResources = Resources.TextureResources;
	}
}

bool FMediaIOCoreTextureSampleBase::ApplyColorConversion(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InSrcTexture, FTextureRHIRef& InDstTexture)
{
	if (CachedOCIOResources)
	{
		FRDGBuilder GraphBuilder(RHICmdList);
		
		{
			RDG_EVENT_SCOPE_STAT(GraphBuilder, MediaIO_ColorConversion, "MediaIO_ColorConversion");

			const FRDGTextureRef ColorConversionInput = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(InSrcTexture, TEXT("MediaTextureResourceColorConverisonInputRT")));
			const FRDGTextureRef ColorConversionOutput = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(InDstTexture, TEXT("MediaTextureResourceColorConverisonOutputRT")));

			constexpr float DefaultDisplayGamma = 1;
		
			FOpenColorIORendering::AddPass_RenderThread(GraphBuilder,
				FScreenPassViewInfo(),
				GMaxRHIFeatureLevel,
				FScreenPassTexture(ColorConversionInput),
				FScreenPassRenderTarget(ColorConversionOutput, ERenderTargetLoadAction::EClear),
				*CachedOCIOResources,
				DefaultDisplayGamma);
		}

		GraphBuilder.Execute();

		return true;
	}
	
	return false;
}

#if WITH_ENGINE
IMediaTextureSampleConverter* FMediaIOCoreTextureSampleBase::GetMediaTextureSampleConverter()
{
	return Converter.Get();
}

FRHITexture* FMediaIOCoreTextureSampleBase::GetTexture() const
{
	return Texture.GetReference();
}

IMediaTextureSampleColorConverter* FMediaIOCoreTextureSampleBase::GetMediaTextureSampleColorConverter()
{
	if (ColorConversionSettings && ColorConversionSettings->IsValid())
	{
		return this;
	}
	return nullptr;
}
#endif

void FMediaIOCoreTextureSampleBase::SetTexture(TRefCountPtr<FRHITexture> InRHITexture)
{
	Texture = MoveTemp(InRHITexture);
}

void FMediaIOCoreTextureSampleBase::SetDestructionCallback(TFunction<void(TRefCountPtr<FRHITexture>)> InDestructionCallback)
{
	DestructionCallback = InDestructionCallback;
}

EPixelFormat FMediaIOCoreTextureSampleBase::GetPixelFormat()
{
	switch (GetFormat())
	{
	case EMediaTextureSampleFormat::FloatRGBA:
		return PF_FloatRGBA;
	case EMediaTextureSampleFormat::CharBGR10A2:
	{
		if (GetEncodingType() != UE::Color::EEncoding::Linear)
		{
			return PF_FloatRGB;
		}
		return PF_FloatRGBA;
	}
	default:
		return PF_B8G8R8A8;
	}
}

void FMediaIOCoreTextureSampleBase::ShutdownPoolable()
{
	if (DestructionCallback)
	{
		DestructionCallback(Texture);
	}

	FreeSample();
}

const FMatrix& FMediaIOCoreTextureSampleBase::GetYUVToRGBMatrix() const
{
	using namespace UE::Color;

	const FColorSpace& SourceColorSpace = GetSourceColorSpace();

	if (SourceColorSpace.Equals(FColorSpace::GetSRGB()))
	{
		return MediaShaders::YuvToRgbRec709Scaled;
	}
	
	if (SourceColorSpace.Equals(UE::Color::FColorSpace::GetRec2020()))
	{
		return MediaShaders::YuvToRgbRec2020Scaled;
	}

	return MediaShaders::YuvToRgbRec709Scaled;
}

bool FMediaIOCoreTextureSampleBase::IsOutputSrgb() const
{
	if (ColorConversionSettings && ColorConversionSettings->IsValid())
	{
		return false; // Don't apply gamma correction as it will be handled by the OCIO conversion.
	}

	if (SourceColorSettings.IsValid())
	{
		const UE::Color::EEncoding EncodingOverride = SourceColorSettings->GetEncodingOverride();

		if (EncodingOverride != UE::Color::EEncoding::None)
		{
			return EncodingOverride == UE::Color::EEncoding::sRGB;
		}
	}

	return false;
}

const UE::Color::FColorSpace& FMediaIOCoreTextureSampleBase::GetSourceColorSpace() const
{
	if (SourceColorSettings.IsValid())
	{
		return SourceColorSettings->GetColorSpaceOverride(UE::Color::FColorSpace::GetSRGB());
	}

	return IMediaTextureSample::GetSourceColorSpace();
}

bool FMediaIOCoreTextureSampleBase::HasEncodingOverride() const
{
	return SourceColorSettings.IsValid() ? SourceColorSettings->HasEncodingOverride() : false;
}

bool FMediaIOCoreTextureSampleBase::HasColorSpaceOverride() const
{
	return SourceColorSettings.IsValid() ? SourceColorSettings->HasColorSpaceOverride() : false;
}

UE::Color::EEncoding FMediaIOCoreTextureSampleBase::GetEncodingType() const
{
	if (ColorConversionSettings && ColorConversionSettings->IsValid())
	{
		return UE::Color::EEncoding::Linear;
	}

	if (SourceColorSettings.IsValid())
	{
		const UE::Color::EEncoding EncodingOverride = SourceColorSettings->GetEncodingOverride();

		if (EncodingOverride != UE::Color::EEncoding::None)
		{
			return EncodingOverride;
		}
	}

	return IMediaTextureSample::GetEncodingType();
}

UE::Color::EChromaticAdaptationMethod FMediaIOCoreTextureSampleBase::GetChromaticAdapationMethod() const
{
	if (SourceColorSettings.IsValid())
	{
		return SourceColorSettings->GetChromaticAdaptationMethod();
	}

	return IMediaTextureSample::GetChromaticAdapationMethod();
}

float FMediaIOCoreTextureSampleBase::GetHDRNitsNormalizationFactor() const
{
	// OCIO handles color conversion itself.
	if (ColorConversionSettings && ColorConversionSettings->IsValid())
	{
		return 1.0f;
	}

	const UE::Color::EEncoding CurrentEncoding = GetEncodingType();

	// User override
	// Note that None encodings automatically reset the reference white to None upstream.
	if (SourceColorSettings.IsValid())
	{
		const UE::Color::EReferenceWhite UserReferenceWhite = SourceColorSettings->GetReferenceWhiteOverride();
		if (UserReferenceWhite != UE::Color::EReferenceWhite::None)
		{
			return UE::Color::GetReferenceWhiteLinearScale(CurrentEncoding, UserReferenceWhite);
		}
	}

	// MediaIO default: HDR streams follow BT.2408.
	if (CurrentEncoding == UE::Color::EEncoding::ST2084 || CurrentEncoding == UE::Color::EEncoding::HLG)
	{
		return UE::Color::GetReferenceWhiteLinearScale(CurrentEncoding, UE::Color::EReferenceWhite::BT2408);
	}

	return 1.0f;
}

// Deprecated
UE::Color::EColorSpace FMediaIOCoreTextureSampleBase::GetColorSpaceType() const
{
	return UE::Color::EColorSpace::None;
}
