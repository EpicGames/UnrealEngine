// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaIOCoreWaitVsyncThread.h"
#include "IMediaIOCoreHardwareSync.h"

#include "Engine/GameEngine.h"
#include "HAL/Event.h"
#include "HAL/RunnableThread.h"
#include "Templates/Atomic.h"

#include "MediaIOCoreModule.h"

FMediaIOCoreWaitVSyncThread::FMediaIOCoreWaitVSyncThread(TSharedPtr<IMediaIOCoreHardwareSync> InHardwareSync)
	: HardwareSync(InHardwareSync)
	, bWaitingForSignal(false)
	, bAlive(false)
{
	const bool bIsManualReset = false;
	WaitVSync = FPlatformProcess::GetSynchEventFromPool(bIsManualReset);
}

bool FMediaIOCoreWaitVSyncThread::Init()
{
	bAlive.Store(true);
	return true;
}

uint32 FMediaIOCoreWaitVSyncThread::Run()
{
	while (!IsEngineExitRequested() && bAlive.Load() && HardwareSync->IsValid())
	{
		// wait for event
		HardwareSync->WaitVSync();

		if (!bWaitingForSignal.Load() && bAlive)
		{
			UE_LOGF(LogMediaIOCore, Error, "The Engine couldn't run fast enough to keep up with the VSync.");
		}

		WaitVSync->Trigger();
	}

	return 0;
}

void FMediaIOCoreWaitVSyncThread::Stop()
{
	bAlive.Store(false);
}

void FMediaIOCoreWaitVSyncThread::Exit()
{
	FPlatformProcess::ReturnSynchEventToPool(WaitVSync);
}

bool FMediaIOCoreWaitVSyncThread::Wait_GameOrRenderThread()
{
	bool bResult = true;
	if (bAlive.Load())
	{
		bWaitingForSignal.Store(true);

		int32 WaitTime = 100; //ms
		bResult = WaitVSync->Wait(WaitTime);

		//Thread could have been stopped during the wait
		if (bAlive.Load())
		{
			WaitVSync->Reset();

			bWaitingForSignal.Store(false);
			if (!bResult)
			{
				UE_LOGF(LogMediaIOCore, Error, "Lost VSync signal.");
			}
		}
	}

	return bResult;
}
