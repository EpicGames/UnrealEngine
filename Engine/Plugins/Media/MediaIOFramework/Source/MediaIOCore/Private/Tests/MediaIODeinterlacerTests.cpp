// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "MediaIOCoreEncodeTime.h"
#include "MediaIOCoreDeinterlacer.h"

namespace UE::MediaIOTests
{
	constexpr uint8 FIRST_LINE_VALUE = 5;
	constexpr uint8 SECOND_LINE_VALUE = 10;

	TArray<uint8> CreateBuffer()
	{
		TArray<uint8> Buffer;
		Buffer.Reserve(1080 * 1920);
		
		for (uint32 YIndex = 0; YIndex < 1080; YIndex++)
		{
			for (uint32 XIndex = 0; XIndex < 1920; XIndex++)
			{
				uint8 Value = FIRST_LINE_VALUE;
				if ((YIndex % 2) != 0)
				{
					Value = SECOND_LINE_VALUE;
				}

				for (uint32 RGBAIndex = 0; RGBAIndex < 4; RGBAIndex++)
				{
					Buffer.Add(Value);
				}
			}
		}

		return Buffer;
	}
	
	constexpr uint32 TestWidth = 1920;
	constexpr uint32 TestHeight = 1080;
	constexpr uint32 TestStride = TestWidth * 4;

	// FVideoFrame holds FrameRate and Timecode by const-ref, so these must outlive any
	// FVideoFrame instance returned from MakeFrameInfo.
	static const FFrameRate TestFrameRate(30, 1);
	static const TOptional<FTimecode> TestTimecode;

	UE::MediaIOCore::FVideoFrame MakeFrameInfo(const TArray<uint8>& Buffer)
	{
		return UE::MediaIOCore::FVideoFrame
		{
			Buffer.GetData(),
			(uint32)Buffer.Num(),
			TestStride,
			TestWidth,
			TestHeight,
			EMediaTextureSampleFormat::CharBGRA,
			FTimespan::FromSeconds(0),
			TestFrameRate,
			TestTimecode,
			nullptr
		};
	}

	void TestBobDeinterlace(FAutomationTestBase& Test)
	{
		TArray<uint8> Buffer = CreateBuffer();
		UE::MediaIOCore::FVideoFrame FrameInfo = MakeFrameInfo(Buffer);

		UE::MediaIOCore::FBobDeinterlacer Deinterlacer(UE::MediaIOCore::FDeinterlacer::FOnAcquireSample_AnyThread::CreateLambda([](){ return MakeShared<FMediaIOCoreTextureSampleBase>(); }), EMediaIOInterlaceFieldOrder::TopFieldFirst);
		TArray<TSharedRef<FMediaIOCoreTextureSampleBase>> DeinterlacedSamples = Deinterlacer.Deinterlace(FrameInfo);

		Test.TestEqual(TEXT("Samples were created"), DeinterlacedSamples.Num(), 2);
		Test.TestEqual(TEXT("First sample is full height"), DeinterlacedSamples[0]->GetDim().Y, (int32)TestHeight);
		Test.TestEqual(TEXT("Second sample is full height"), DeinterlacedSamples[1]->GetDim().Y, (int32)TestHeight);

		uint8* Buffer1 = (uint8*)DeinterlacedSamples[0]->GetBuffer();
		uint8* Buffer2 = (uint8*)DeinterlacedSamples[1]->GetBuffer();

		Test.TestEqual(TEXT("First sample first line is correct"), *Buffer1, FIRST_LINE_VALUE);
		Test.TestEqual(TEXT("First sample Second line is correct"), *(Buffer1 + TestStride), FIRST_LINE_VALUE);

		Test.TestEqual(TEXT("Second sample first line is correct"), *Buffer2, SECOND_LINE_VALUE);
		Test.TestEqual(TEXT("Second sample Second line is correct"), *(Buffer2 + TestStride), SECOND_LINE_VALUE);

		// Bob must offset Odd sample's Time by one field period so the renderer sees non-overlapping intervals.
		Test.TestTrue(TEXT("Odd sample Time is offset from Even"), DeinterlacedSamples[1]->GetTime().Time > DeinterlacedSamples[0]->GetTime().Time);
	}

	void TestBlendDeinterlace(FAutomationTestBase& Test)
	{
		TArray<uint8> Buffer = CreateBuffer();
		UE::MediaIOCore::FVideoFrame FrameInfo = MakeFrameInfo(Buffer);

		UE::MediaIOCore::FBlendDeinterlacer Deinterlacer(UE::MediaIOCore::FDeinterlacer::FOnAcquireSample_AnyThread::CreateLambda([]() { return MakeShared<FMediaIOCoreTextureSampleBase>(); }), EMediaIOInterlaceFieldOrder::TopFieldFirst);
		TArray<TSharedRef<FMediaIOCoreTextureSampleBase>> DeinterlacedSamples = Deinterlacer.Deinterlace(FrameInfo);

		Test.TestEqual(TEXT("One Sample was created"), DeinterlacedSamples.Num(), 1);
		Test.TestEqual(TEXT("Sample is full height"), DeinterlacedSamples[0]->GetDim().Y, (int32)TestHeight);

		uint8* OutBuffer = (uint8*)DeinterlacedSamples[0]->GetBuffer();
		const uint8 Averaged = (uint8)FMath::DivideAndRoundUp(FIRST_LINE_VALUE + SECOND_LINE_VALUE, 2);

		Test.TestEqual(TEXT("Y=0 is averaged across fields"), *OutBuffer, Averaged);
		Test.TestEqual(TEXT("Y=1 mirrors Y=0 (line-doubled blend)"), *(OutBuffer + TestStride), Averaged);
	}

	void TestDiscardDeinterlace(FAutomationTestBase& Test)
	{
		TArray<uint8> Buffer = CreateBuffer();
		UE::MediaIOCore::FVideoFrame FrameInfo = MakeFrameInfo(Buffer);

		UE::MediaIOCore::FDiscardDeinterlacer Deinterlacer(UE::MediaIOCore::FDeinterlacer::FOnAcquireSample_AnyThread::CreateLambda([]() { return MakeShared<FMediaIOCoreTextureSampleBase>(); }), EMediaIOInterlaceFieldOrder::TopFieldFirst);
		TArray<TSharedRef<FMediaIOCoreTextureSampleBase>> DeinterlacedSamples = Deinterlacer.Deinterlace(FrameInfo);

		Test.TestEqual(TEXT("One sample was created"), DeinterlacedSamples.Num(), 1);
		Test.TestEqual(TEXT("Sample is full height"), DeinterlacedSamples[0]->GetDim().Y, (int32)TestHeight);

		uint8* Buffer1 = (uint8*)DeinterlacedSamples[0]->GetBuffer();
		Test.TestEqual(TEXT("First sample first line is correct"), *Buffer1, FIRST_LINE_VALUE);
		Test.TestEqual(TEXT("First sample Second line is correct"), *(Buffer1 + TestStride), FIRST_LINE_VALUE);
	}

	void TestDiscardDeinterlaceBottomFieldOrder(FAutomationTestBase& Test)
	{
		TArray<uint8> Buffer = CreateBuffer();
		UE::MediaIOCore::FVideoFrame FrameInfo = MakeFrameInfo(Buffer);

		UE::MediaIOCore::FDiscardDeinterlacer Deinterlacer(UE::MediaIOCore::FDeinterlacer::FOnAcquireSample_AnyThread::CreateLambda([]() { return MakeShared<FMediaIOCoreTextureSampleBase>(); }), EMediaIOInterlaceFieldOrder::BottomFieldFirst);
		TArray<TSharedRef<FMediaIOCoreTextureSampleBase>> DeinterlacedSamples = Deinterlacer.Deinterlace(FrameInfo);

		Test.TestEqual(TEXT("One sample was created"), DeinterlacedSamples.Num(), 1);
		Test.TestEqual(TEXT("Sample is full height"), DeinterlacedSamples[0]->GetDim().Y, (int32)TestHeight);

		uint8* Buffer1 = (uint8*)DeinterlacedSamples[0]->GetBuffer();
		Test.TestEqual(TEXT("First sample first line is correct"), *Buffer1, SECOND_LINE_VALUE);
		Test.TestEqual(TEXT("First sample Second line is correct"), *(Buffer1 + TestStride), SECOND_LINE_VALUE);
	}
}

DEFINE_SPEC(FMediaIODeinterlacerTests, "Plugins.MediaIO.Deinterlace", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
void FMediaIODeinterlacerTests::Define()
{
	It("BobDeinterlace", [this]()
		{
			UE::MediaIOTests::TestBobDeinterlace(*this);
		});

	It("BlendDeinterlace", [this]()
		{
			UE::MediaIOTests::TestBlendDeinterlace(*this);
		});

	It("DiscardDeinterlace", [this]()
		{
			UE::MediaIOTests::TestDiscardDeinterlace(*this);
		});

	It("DiscardDeinterlaceBottomFieldOrder", [this]()
		{
			UE::MediaIOTests::TestDiscardDeinterlaceBottomFieldOrder(*this);
		});
}