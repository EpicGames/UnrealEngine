// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TimeSynchronizableMediaSource.h"
#include "BaseMediaSourceColorSettings.h"
#include "Containers/UnrealString.h"
#include "Engine/TextureDefines.h"
#include "HAL/Platform.h"
#include "MediaIOCoreDefinitions.h"
#include "MediaIOCoreDeinterlacer.h"
#include "OpenColorIOColorSpace.h"
#include "OpenColorIORendering.h"
#include "UObject/NameTypes.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"


#include "CaptureCardMediaSource.generated.h"

#define UE_API MEDIAIOCORE_API

namespace UE::CaptureCardMediaSource
{
	static FName RenderJIT("RenderJIT");
	static FName Framelock("Framelock");
	static FName EvaluationType("EvaluationType");
	static FName Deinterlacer("Deinterlacer");
	static FName InterlaceFieldOrder("InterlaceFieldOrder");
	static FName SourceColorSettingsOption("SourceColorSettings");
	static FName OpenColorIOSettings("OpenColorIOSettings");

	// 5.8 deprecated options
	UE_DEPRECATED(5.8, "OverrideSourceEncoding is deprecated.")
	static FName OverrideSourceEncoding("OverrideSourceEncoding_DEPRECATED");

	UE_DEPRECATED(5.8, "SourceEncoding is deprecated.")
	static FName SourceEncoding("SourceEncoding_DEPRECATED");

	UE_DEPRECATED(5.8, "OverrideSourceColorSpace is deprecated.")
	static FName OverrideSourceColorSpace("OverrideSourceColorSpace_DEPRECATED");

	UE_DEPRECATED(5.8, "SourceColorSpace is deprecated.")
	static FName SourceColorSpace("SourceColorcSpace_DEPRECATED");
}


UENUM(meta = (Deprecated = "5.8", DeprecationMessage = "EMediaIOCoreSourceEncoding is deprecated. This functionality is now available with EMediaSourceEncoding in BaseMediaSourceColorSettings.h"))
enum class UE_DEPRECATED(5.8, "EMediaIOCoreSourceEncoding is deprecated. This functionality is now available with EMediaSourceEncoding in BaseMediaSourceColorSettings.") EMediaIOCoreSourceEncoding : uint8
{
	Linear		= 1 UMETA(DisplayName = "Linear"),
	sRGB		= 2 UMETA(DisplayName = "sRGB"),
	ST2084		= 3 UMETA(DisplayName = "ST 2084/PQ"),
	SLog3		= 12 UMETA(DisplayName = "SLog3"),
	MAX,
};

/**
 * Base class for media sources that are coming from a capture card.
 */
UCLASS(MinimalAPI, Abstract)
class UCaptureCardMediaSource : public UTimeSynchronizableMediaSource
{
	GENERATED_BODY()
	
public:

	UE_API UCaptureCardMediaSource();

public:
	/** Should use JITR technique? It enables late sample picking for render. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Synchronization, meta = (DisplayName = "Just-In-Time Rendering"))
	bool bRenderJIT = true;

	/** Should wait for some time until requested frame arrives? Requires JIT rendering. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Synchronization, meta = (DisplayName = "Framelock"))
	bool bFramelock = false;

	/** Sample evaluation type. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Synchronization, meta = (DisplayName = "Sample Evaluation Type"))
	EMediaIOSampleEvaluationType EvaluationType = EMediaIOSampleEvaluationType::PlatformTime;

	/**
	 * How interlaced video should be treated.
	 */
	UPROPERTY(BlueprintReadOnly, Instanced, EditAnywhere, Category = "Video")
	TObjectPtr<UVideoDeinterlacer> Deinterlacer;

	/**
	 * The order in which interlace fields should be copied.
	 */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "Video")
	EMediaIOInterlaceFieldOrder InterlaceFieldOrder = EMediaIOInterlaceFieldOrder::TopFieldFirst;

	/**
	 * OpenColorIO transform settings used for applying a color conversion override to the incoming source.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "ColorManagement", meta = (DisplayName = "OpenColorIO Override"))
	FOpenColorIOColorConversionSettings ColorConversionSettings;

	/** Manual definition of media source color space & encoding. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "ColorManagement", meta = (ShowOnlyInnerProperties))
	FMediaSourceColorSettings SourceColorSettings;

private:
	/** Native source color settings. */
	TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> NativeSourceColorSettings;

public:
	struct FOpenColorIODataContainer : public IMediaOptions::FDataContainer
	{
		FOpenColorIOColorConversionSettings ColorConversionSettings;
	};

public:
	//~ IMediaOptions interface
	using Super::GetMediaOption;
	UE_API virtual FString GetMediaOption(const FName& Key, const FString& DefaultValue) const override;
	UE_API virtual int64 GetMediaOption(const FName& Key, int64 DefaultValue) const override;
	UE_API virtual bool GetMediaOption(const FName& Key, bool bDefaultValue) const override;
	UE_API virtual TSharedPtr<FDataContainer, ESPMode::ThreadSafe> GetMediaOption(const FName& Key, const TSharedPtr<FDataContainer, ESPMode::ThreadSafe>& DefaultValue) const override;
	UE_API virtual bool HasMediaOption(const FName& Key) const override;

public:
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;
#if WITH_EDITOR
	UE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	UE_API virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;

	/**
	 * Cache the configuration the player auto-detected from the live signal so the editor's
	 * Media Profile info panel can show the actual format instead of the stored default.
	 * Game-thread only.
	 */
	UE_API void SetLastDetectedConfiguration(const FMediaIOConfiguration& InConfiguration);

	/**
	 * Drop the cached detected configuration. Called from a player's Close path so the panel
	 * reverts to "Auto (no signal)" instead of showing stale R/F/S after disconnect.
	 * Game-thread only.
	 */
	UE_API void ClearLastDetectedConfiguration();

	bool HasLastDetectedConfiguration() const { return bHasLastDetectedConfiguration; }
	const FMediaIOConfiguration& GetLastDetectedConfiguration() const { return LastDetectedConfiguration; }

	/** Broadcast on the game thread whenever the detected configuration is set or cleared, so editor UIs can refresh. */
	FSimpleMulticastDelegate& OnLastDetectedConfigurationChanged() { return LastDetectedConfigurationChanged; }
#endif //WITH_EDITOR

private:
#if WITH_EDITORONLY_DATA
	/** The configuration last reported by a running player's auto-detection. Transient, never serialized. */
	UPROPERTY(Transient)
	FMediaIOConfiguration LastDetectedConfiguration;

	/** True once a player has published a detected configuration this session. */
	UPROPERTY(Transient)
	bool bHasLastDetectedConfiguration = false;
#endif

#if WITH_EDITOR
	/** Notifies editor listeners (e.g. the Media Profile info panel) when the detected configuration changes. */
	FSimpleMulticastDelegate LastDetectedConfigurationChanged;
#endif

public:
	//~ Deprecated properties
#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "This property is deprecated. Please use SourceColorSettings instead.")
	UPROPERTY(meta = (ScriptName = "IsOverrideSourceEncoding", DeprecatedProperty, DeprecationMessage = "This property is deprecated. Please use SourceColorSettings instead."))
	bool bOverrideSourceEncoding_DEPRECATED = true;

	UE_DEPRECATED(5.8, "This property is deprecated. Please use SourceColorSettings instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "This property is deprecated. Please use SourceColorSettings instead."))
	EMediaIOCoreSourceEncoding OverrideSourceEncoding_DEPRECATED = EMediaIOCoreSourceEncoding::Linear;

	UE_DEPRECATED(5.8, "This property is deprecated. Please use SourceColorSettings instead.")
	UPROPERTY(meta = (ScriptName = "IsOverrideSourceColorSpace", DeprecatedProperty, DeprecationMessage = "This property is deprecated. Please use SourceColorSettings instead."))
	bool bOverrideSourceColorSpace_DEPRECATED = true;

	UE_DEPRECATED(5.8, "This property is deprecated. Please use SourceColorSettings instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "This property is deprecated. Please use SourceColorSettings instead."))
	ETextureColorSpace OverrideSourceColorSpace_DEPRECATED = ETextureColorSpace::TCS_sRGB;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
};

#undef UE_API
