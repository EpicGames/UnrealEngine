// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaCapture.h"

#include "ImageWriteTypes.h"
#include "FileMediaCapture.generated.h"

#define UE_API MEDIAIOCORE_API

/**
 * 
 */
UCLASS(MinimalAPI)
class UFileMediaCapture : public UMediaCapture
{
	GENERATED_BODY()

protected:
	UE_API virtual void OnFrameCaptured_RenderingThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, void* InBuffer, int32 Width, int32 Height, int32 BytesPerRow) override;
	UE_API virtual bool InitializeCapture() override;

private:
	UE_API void CacheMediaOutputValues();

private:
	FString BaseFilePathName;
	EImageFormat ImageFormat;
	TFunction<void(bool)> OnCompleteWrapper;
	bool bOverwriteFile;
	int32 CompressionQuality;
	bool bAsync;
};

#undef UE_API
