// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MediaOutput.h"

#include "Engine/RendererSettings.h"
#include "ImageWriteBlueprintLibrary.h"

#include "FileMediaOutput.generated.h"

#define UE_API MEDIAIOCORE_API


/** Texture format supported by UFileMediaOutput. */
UENUM()
enum class EFileMediaOutputPixelFormat
{
	B8G8R8A8					UMETA(DisplayName = "8bit RGBA"),
	FloatRGBA					UMETA(DisplayName = "Float RGBA"),
};


/**
 * Output information for a file media capture.
 * @note	'Frame Buffer Pixel Format' must be set to at least 8 bits of alpha to enabled the Key.
 * @note	'Enable alpha channel support in post-processing' must be set to 'Allow through tonemapper' to enabled the Key.
 */
UCLASS(MinimalAPI, BlueprintType)
class UFileMediaOutput : public UMediaOutput
{
	GENERATED_BODY()

public:
	UE_API UFileMediaOutput();

public:
	/** Options on how to save the images. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="File")
	FImageWriteOptions WriteOptions;

	/** The file path for the images. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="File", meta=(RelativePath))
	FDirectoryPath FilePath;

	/** The base file name of the images. The frame number will be append to the base file name. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="File", meta=(RelativePath))
	FString BaseFileName;

	/** Use the default back buffer size or specify a specific size to capture. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Media", meta = (InlineEditConditionToggle))
	bool bOverrideDesiredSize;

	/** Use the default back buffer size or specify a specific size to capture. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Media", meta=(EditCondition="bOverrideDesiredSize"))
	FIntPoint DesiredSize;

	/** Use the default back buffer pixel format or specify a specific the pixel format to capture. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Media", meta = (InlineEditConditionToggle))
	bool bOverridePixelFormat;

	/** Use the default back buffer pixel format or specify a specific the pixel format to capture. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Media", meta=(EditCondition="bOverridePixelFormat"))
	EFileMediaOutputPixelFormat DesiredPixelFormat;

	/** Invert the alpha for formats that support alpha. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Media")
	bool bInvertAlpha;

	//~ UMediaOutput interface
public:
	UE_API virtual bool Validate(FString& FailureReason) const override;
	UE_API virtual FIntPoint GetRequestedSize() const override;
	UE_API virtual EPixelFormat GetRequestedPixelFormat() const override;
	UE_API virtual EMediaCaptureConversionOperation GetConversionOperation(EMediaCaptureSourceType InSourceType) const override;

#if WITH_EDITOR
	virtual FString GetDescriptionString() const override;
	virtual void GetDetailsPanelInfoElements(TArray<FInfoElement>& OutInfoElements) const override;
#endif //WITH_EDITOR
	
protected:
	UE_API virtual UMediaCapture* CreateMediaCaptureImpl() override;
	//~ End UMediaOutput interface
};

#undef UE_API
