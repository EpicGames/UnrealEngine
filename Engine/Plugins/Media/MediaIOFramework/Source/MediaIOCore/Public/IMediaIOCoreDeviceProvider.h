// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MediaIOCoreDefinitions.h"

#define UE_API MEDIAIOCORE_API

class UMediaSource;

class IMediaIOCoreDeviceProvider
{
public:
	virtual FName GetFName() = 0;

	virtual TArray<FMediaIOConnection> GetConnections() const = 0;
	virtual TArray<FMediaIOConfiguration> GetConfigurations() const = 0;
	virtual TArray<FMediaIOConfiguration> GetConfigurations(bool bAllowInput, bool bAllowOutput) const = 0;
	virtual TArray<FMediaIODevice> GetDevices() const = 0;
	virtual TArray<FMediaIOMode> GetModes(const FMediaIODevice& InDevice, bool bInOutput) const = 0;
	virtual TArray<FMediaIOInputConfiguration> GetInputConfigurations() const = 0;
	virtual TArray<FMediaIOOutputConfiguration> GetOutputConfigurations() const = 0;
	virtual TArray<FMediaIOVideoTimecodeConfiguration> GetTimecodeConfigurations() const = 0;

	virtual FMediaIOConfiguration GetDefaultConfiguration() const = 0;
	virtual FMediaIOMode GetDefaultMode() const = 0;
	virtual FMediaIOInputConfiguration GetDefaultInputConfiguration() const = 0;
	virtual FMediaIOOutputConfiguration GetDefaultOutputConfiguration() const = 0;
	virtual FMediaIOVideoTimecodeConfiguration GetDefaultTimecodeConfiguration() const = 0;

	virtual UMediaSource* CreateMediaSource(const FMediaIOConfiguration& InConfiguration,
		UObject* Outer) const { return nullptr; }

	UE_API virtual FText ToText(const FMediaIOConfiguration& InConfiguration, bool bInIsAutoDetected = false) const;
	UE_API virtual FText ToText(const FMediaIOConnection& InConnection) const;
	UE_API virtual FText ToText(const FMediaIODevice& InDevice) const;
	UE_API virtual FText ToText(const FMediaIOMode& InMode) const;
	UE_API virtual FText ToText(const FMediaIOInputConfiguration& InMode) const;
	UE_API virtual FText ToText(const FMediaIOOutputConfiguration& InMode) const;

	static UE_API FText GetTransportName(EMediaIOTransportType InLinkType, EMediaIOQuadLinkTransportType InQuadLinkType);

#if WITH_EDITOR
	virtual bool ShowInputTransportInSelector() const { return true; }
	virtual bool ShowOutputTransportInSelector() const { return true; }
	virtual bool ShowInputKeyInSelector() const { return true; }
	virtual bool ShowOutputKeyInSelector() const { return true; }
	virtual bool ShowReferenceInSelector() const { return true; }
#endif
};

#undef UE_API
