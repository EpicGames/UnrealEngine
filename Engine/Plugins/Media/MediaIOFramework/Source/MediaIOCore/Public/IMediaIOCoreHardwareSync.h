// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
* Interface to Hardware Sync Device
*/

class IMediaIOCoreHardwareSync
{
public:
	/** Is hardware valid, and ready to synchronize. */
	virtual bool IsValid() const = 0;

	/** Wait on hardware synchronization. */
	virtual bool WaitVSync() = 0;

public:
	/** Virtual destructor for the interface. */ 
	virtual ~IMediaIOCoreHardwareSync() = default;
};
