// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

#define UE_API MEDIAIOCORE_API


class IMediaIOCoreDeviceProvider;


/**
 * Definition the MediaIOCore module.
 */
class IMediaIOCoreModule : public IModuleInterface
{
public:
	static UE_API bool IsAvailable();
	static UE_API IMediaIOCoreModule& Get();

public:
	virtual void RegisterDeviceProvider(IMediaIOCoreDeviceProvider* InProvider) = 0;
	virtual void UnregisterDeviceProvider(IMediaIOCoreDeviceProvider* InProvider) = 0;
	virtual IMediaIOCoreDeviceProvider* GetDeviceProvider(FName InProviderName) = 0;
	virtual TConstArrayView<IMediaIOCoreDeviceProvider*> GetDeviceProviders() const = 0;
};

#undef UE_API
