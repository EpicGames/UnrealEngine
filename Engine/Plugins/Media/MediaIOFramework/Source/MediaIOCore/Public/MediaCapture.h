// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/Object.h"

#include "AudioDeviceHandle.h"
#include "Containers/SpscQueue.h"
#include "HAL/CriticalSection.h"
#include "MediaOutput.h"
#include "Misc/Timecode.h"
#include "OpenColorIOColorSpace.h"
#include "OpenColorIORendering.h"
#include "OrderedAsyncGate.h"
#include "PixelFormat.h"
#include "RenderGraphUtils.h"
#include "RHI.h"
#include "RHIResources.h"
#include "Tasks/Task.h"
#include "Templates/PimplPtr.h"

#include <atomic>

#include "MediaCapture.generated.h"

#define UE_API MEDIAIOCORE_API

class FViewport;
class FSceneViewport;
class FTextureRenderTargetResource;
class UTextureRenderTarget2D;
class FRHICommandListImmediate;
class FRHIGPUTextureReadback;
class FRHIGPUBufferReadback;
class FRDGBuilder;
class UMediaCapture;
class SWindow;
class ISlateViewportProvider;

#if WITH_EDITOR
struct FAnalyticsEventAttribute;
#endif

namespace UE::MediaCaptureData
{
	class FFrameManager;
	class FCaptureFrame;
	class FMediaCaptureHelper;
	struct FCaptureFrameArgs;
	class FTextureCaptureFrame;
	class FBufferCaptureFrame;
	class FSyncPointWatcher;
	struct FPendingCaptureData;
}

namespace UE::MediaCapture::Private
{
	class FCaptureSource;
}

namespace UE::MediaCapture
{
	struct FRenderPass;
	class FRenderPipeline;
}

/**
 * Possible states of media capture.
 */
UENUM(BlueprintType)
enum class EMediaCaptureState : uint8
{
	/** Unrecoverable error occurred during capture. */
	Error,

	/** Media is currently capturing. */
	Capturing,

	/** Media is being prepared for capturing. */
	Preparing,

	/** Capture has been stopped but some frames may need to be process. */
	StopRequested,

	/** Capture has been stopped. */
	Stopped,
};

/**
 * Possible resource type the MediaCapture can create based on the conversion operation.
 */
UENUM()
enum class EMediaCaptureResourceType : uint8
{
	/** Texture resources are used for the readback. */
	Texture,

	/** RWBuffer resources are used for the readback. */
	Buffer,
};

/**
 * Base class of additional data that can be stored for each requested capture.
 */
class FMediaCaptureUserData
{
};

/**
 * Type of cropping 
 */
UENUM(BlueprintType)
enum class EMediaCaptureCroppingType : uint8
{
	/** Do not crop the captured image. */
	None,
	/** Keep the center of the captured image. */
	Center,
	/** Keep the top left corner of the captured image. */
	TopLeft,
	/** Use the StartCapturePoint and the size of the MediaOutput to keep of the captured image. */
	Custom,
};

/**
 * Action when overrun occurs
 */
UENUM(BlueprintType)
enum class EMediaCaptureOverrunAction : uint8
{
	/** Flush rendering thread such that all scheduled commands are executed. */
	Flush,
	/** Skip capturing a frame if readback is trailing too much. */
	Skip,
};

/**
 * Description of resource to be captured when in rhi capture / immediate mode.
 */
struct FRHICaptureResourceDescription
{
	FIntPoint ResourceSize = FIntPoint::ZeroValue;
	EPixelFormat PixelFormat = EPixelFormat::PF_Unknown;
};

UENUM(BlueprintType)
enum class EMediaCapturePhase : uint8
{
	BeforePostProcessing, // Will happen after TSR in order to get a texture of the right size.
	AfterMotionBlur,
	AfterToneMap,
	AfterFXAA,
	PostRender,
	BackBufferReady,
	EndFrame
};

UENUM(BlueprintType)
enum class EMediaCaptureResizeMethod : uint8
{
	/** Leaves the source texture as is, which might be incompatible with the output size, causing an error. */
	None,
	/** Resize the source that is being captured. This will resize the viewport when doing a viewport capture and will resize the render target when capturing a render target. Not valid for immediate capture of RHI resource */
	ResizeSource,
	/** Leaves the source as is, but will add a render pass where we resize the captured texture, leaving the source intact. This is useful when the output size is smaller than the captured source. */
	ResizeInRenderPass,
	/**  Leaves the source as is, but expects the capture pass to fully re-render the source texture to the output texture. */
	ResizeInCapturePass
};


/**
 * Base class of additional data that can be stored for each requested capture.
 */
USTRUCT(BlueprintType)
struct FMediaCaptureOptions
{
	GENERATED_BODY()

	UE_API FMediaCaptureOptions();

public:
	UE_API void PostSerialize(const FArchive& Ar);

	/** Gets the string version of ColorConversionSettings */
	UE_API FString GetColorConversionSettingsString() const;

	/** If source texture's size change, auto restart capture to follow source resolution if implementation supports it. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "MediaCapture")
	bool bAutoRestartOnSourceSizeChange = false;

	/** Action to do when game thread overruns render thread and all frames are in flights being captured / readback. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "MediaCapture")
	EMediaCaptureOverrunAction OverrunAction = EMediaCaptureOverrunAction::Flush;

	/** Crop the captured SceneViewport or TextureRenderTarget2D to the desired size. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="MediaCapture")
	EMediaCaptureCroppingType Crop;

	/** Color conversion settings used by the OCIO conversion pass. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "MediaCapture")
	FOpenColorIOColorConversionSettings ColorConversionSettings;

	/**
	 * Crop the captured SceneViewport or TextureRenderTarget2D to the desired size.
	 * @note Only valid when Crop is set to Custom.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="MediaCapture", meta=(editcondition = "Crop == EMediaCaptureCroppingType::Custom"))
	FIntPoint CustomCapturePoint;
	
	/**
	 * When the capture start, control if and how the source buffer should be resized to the desired size.
	 * @note Only valid when a size is specified by the MediaOutput.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="MediaCapture", meta = (editcondition = "Crop == EMediaCaptureCroppingType::None"))
	EMediaCaptureResizeMethod ResizeMethod = EMediaCaptureResizeMethod::None;

	/**
	 * When the application enters responsive mode, skip the frame capture.
	 * The application can enter responsive mode on mouse down, viewport resize, ...
	 * That is to ensure responsiveness in low FPS situations.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="MediaCapture")
	bool bSkipFrameWhenRunningExpensiveTasks;

	/**
	 * Allows to enable/disable pixel format conversion for the cases where render target is not of the desired pixel format. 
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="MediaCapture")
	bool bConvertToDesiredPixelFormat;

	/**
	 * In some cases when we want to stream irregular render targets containing limited number
	 * of channels (for example RG16f), we would like to force Alpha to 1.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="MediaCapture")
	bool bForceAlphaToOneOnConversion;

	/**
	 * When enabled, editor-only overlays such as actor sprites (player start, light icons, etc.)
	 * are hidden during capture. Has no effect in non-editor builds.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="MediaCapture")
	bool bDisableEditorSpritesDuringCapture = true;

	/**
	 * Whether to apply a linear to sRGB conversion to the texture before outputting. 
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "MediaCapture", meta=(DisplayName="Apply linear to sRGB conversion"))
	bool bApplyLinearToSRGBConversion = false;

	/** Automatically stop capturing after a predetermined number of images. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "MediaCapture")
	bool bAutostopOnCapture;

	/** The number of images to capture*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "MediaCapture", meta=(editcondition="bAutostopOnCapture"))
	int32 NumberOfFramesToCapture;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "MediaCapture")
	EMediaCapturePhase CapturePhase = EMediaCapturePhase::EndFrame;

private:
#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage="bResizeSourceBuffer, please use ResizeMethod instead."))
    bool bResizeSourceBuffer_DEPRECATED = false;
#endif
};

UENUM()
enum class EMediaCaptureRotation : uint8
{
	None UMETA(DisplayName = "0°"),
	CW90 UMETA(DisplayName = "90°"),
	CCW90 UMETA(DisplayName = "-90°"),
	CW180 UMETA(DisplayName = "180°")
};

USTRUCT()
struct FMediaCaptureTransform
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category="Media Texture Capture")
	EMediaCaptureRotation Rotation = EMediaCaptureRotation::None;
	
	UPROPERTY(EditAnywhere, Category="Media Texture Capture")
	bool bFlipHorizontal = false;
	
	UPROPERTY(EditAnywhere, Category="Media Texture Capture")
	bool bFlipVertical = false;
	
	UE_API float GetRotationValue(bool bInDegrees = false) const;
	UE_API FVector2f GetScaleValue() const;
};


/** Delegate signatures */
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FMediaCaptureStateChangedSignature);
DECLARE_MULTICAST_DELEGATE(FMediaCaptureStateChangedSignatureNative);
DECLARE_DELEGATE(FMediaCaptureOutputSynchronization)

/**
 * Abstract base class for media capture.
 *
 * MediaCapture capture the texture of the Render target or the SceneViewport and sends it to an external media device.
 * MediaCapture should be created by a MediaOutput.
 */
UCLASS(MinimalAPI, Abstract, editinlinenew, BlueprintType, hidecategories = (Object))
class UMediaCapture : public UObject
{
	GENERATED_BODY()

public:
	UE_API UMediaCapture();

	/** Default constructor / destructor to use forward declared uniqueptr */
	UE_API virtual ~UMediaCapture();

	/**
	 * Stop the actual capture if there is one.
	 * Then start the capture of a SceneViewport.
	 * If the SceneViewport is destroyed, the capture will stop.
	 * The SceneViewport needs to be of the same size and have the same pixel format as requested by the media output.
	 * @note make sure the size of the SceneViewport doesn't change during capture.
	 * @return True if the capture was successfully started
	 */
	UE_API bool CaptureSceneViewport(TSharedPtr<FSceneViewport>& SceneViewport, FMediaCaptureOptions CaptureOptions);

	/**
	 * Stop the actual capture if there is one.
	 * Then start a capture for a RHI resource.
	 * @note For this to work, it is expected that CaptureImmediate_RenderThread is called periodically with the resource to capture.
	 * @return True if the capture was successfully initialize
	 */
	UE_API bool CaptureRHITexture(const FRHICaptureResourceDescription& ResourceDescription, FMediaCaptureOptions CaptureOptions);

	/**
	 * Stop the current capture if there is one.
	 * Then find and capture every frame from active SceneViewport.
	 * It can only find a SceneViewport when you play in Standalone or in "New Editor Window PIE".
	 * If the active SceneViewport is destroyed, the capture will stop.
	 * The SceneViewport needs to be of the same size and have the same pixel format as requested by the media output.
	 * @return True if the capture was successfully started
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|Output")
	UE_API bool CaptureActiveSceneViewport(FMediaCaptureOptions CaptureOptions);

	/**
	 * Stop the actual capture if there is one.
	 * Then capture every frame for a TextureRenderTarget2D.
	 * The TextureRenderTarget2D needs to be of the same size and have the same pixel format as requested by the media output.
	 * @return True if the capture was successfully started
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|Output")
	UE_API bool CaptureTextureRenderTarget2D(UTextureRenderTarget2D* RenderTarget, FMediaCaptureOptions CaptureOptions);

	/**
	 * Stop the actual capture if there is one.
	 * Then capture every frame for a MediaTexture.
	 * The MediaTexture needs to be of the same size and have the same pixel format as requested by the media output.
	 * @return True if the capture was successfully started
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|Output")
	UE_API bool CaptureMediaTexture(UMediaTexture* InMediaTexture, FMediaCaptureOptions CaptureOptions, float Rotation, const FVector2f& Scale);
	
	/**
	 * Update the current capture with a SceneViewport.
	 * If the SceneViewport is destroyed, the capture will stop.
	 * The SceneViewport needs to be of the same size and have the same pixel format as requested by the media output.
	 * @note make sure the size of the SceneViewport doesn't change during capture.
	 * @return Return true if the capture was successfully updated. If false is returned, the capture was stopped.
	 */
	UE_API bool UpdateSceneViewport(TSharedPtr<FSceneViewport>& SceneViewport);

	/**
	 * Update the current capture with every frame for a TextureRenderTarget2D.
	 * The TextureRenderTarget2D needs to be of the same size and have the same pixel format as requested by the media output.
	 * @return Return true if the capture was successfully updated. If false is returned, the capture was stopped.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|Output")
	UE_API bool UpdateTextureRenderTarget2D(UTextureRenderTarget2D* RenderTarget);
	
	/** 
	 * Captures a resource immediately from the render thread. Used in RHI_RESOURCE capture mode 
	 * @return Return true if the capture was successfully processed.
	 */
	UE_API bool TryCaptureImmediate_RenderThread(FRDGBuilder& GraphBuilder, FRHITexture* InSourceTexture, FIntRect SourceViewRect = FIntRect(0,0,0,0));
	UE_API bool TryCaptureImmediate_RenderThread(FRDGBuilder& GraphBuilder, FRDGTextureRef InSourceTextureRef, FIntRect SourceViewRect = FIntRect(0,0,0,0));

	UE_DEPRECATED(5.5, "This method has been deprecated. Please use TryCaptureImmediate_RenderThread as it returns a boolean to signify that the capture was successfully processed.")
	UE_API void CaptureImmediate_RenderThread(FRDGBuilder& GraphBuilder, FRHITexture* InSourceTexture, FIntRect SourceViewRect = FIntRect(0,0,0,0));

	UE_DEPRECATED(5.5, "This method has been deprecated. Please use TryCaptureImmediate_RenderThread as it returns a boolean to signify that the capture was successfully processed.")
	UE_API void CaptureImmediate_RenderThread(FRDGBuilder& GraphBuilder, FRDGTextureRef InSourceTextureRef, FIntRect SourceViewRect = FIntRect(0,0,0,0));

	/**
	 * Stop the previous requested capture.
	 * @param bAllowPendingFrameToBeProcess	Keep copying the pending frames asynchronously or stop immediately without copying the pending frames.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|Output")
	UE_API void StopCapture(bool bAllowPendingFrameToBeProcess);

	/**
	 * Implementation of StopCapture that allows providing a callback that's invoked when the capture has completely stopped.
	 * Some capture implementations (ie. Aja), need a few frames to completely stop the capture, so this method can be used
	 * to act when the underlying SDK is done shutting down.
	 */
	UE_API void StopCapture_Callback(bool bAllowPendingFrameToBeProcess, FSimpleDelegate Callback);

	/** Get the current state of the capture. */
	UFUNCTION(BlueprintCallable, Category = "Media|Output")
	virtual EMediaCaptureState GetState() const { return MediaState; }

	/** Set the media output. Can only be set when the capture is stopped. */
	UFUNCTION(BlueprintCallable, Category = "Media|Output")
	UE_API void SetMediaOutput(UMediaOutput* InMediaOutput);

	/** Set the audio device to capture. */
	UE_API bool SetCaptureAudioDevice(const FAudioDeviceHandle& InAudioDeviceHandle);

	/** Get the audio device to capture. */
	const FAudioDeviceHandle& GetCaptureAudioDevice() const { return AudioDeviceHandle; }
	
	/** Get the desired size of the current capture. */
	UFUNCTION(BlueprintCallable, Category = "Media|Output")
	FIntPoint GetDesiredSize() const { return DesiredSize; }

	/** Get the desired pixel format of the current capture. */
	UFUNCTION(BlueprintCallable, Category = "Media|Output")
	EPixelFormat GetDesiredPixelFormat() const { return DesiredPixelFormat; }

	/** Get the capture options specified by the user.  */
	const FMediaCaptureOptions& GetDesiredCaptureOptions() const { return DesiredCaptureOptions; }

	/** Check whether this capture has any processing left to do. */
	UE_API virtual bool HasFinishedProcessing() const;

	/** Called when the state of the capture changed. */
	UPROPERTY(BlueprintAssignable, Category = "Media|Output")
	FMediaCaptureStateChangedSignature OnStateChanged;

	/** Set the valid GPU mask of the source buffer/texture being captured. Can be called from any thread. */
	UE_API virtual void SetValidSourceGPUMask(FRHIGPUMask GPUMask);

	/**
	 * Called when the state of the capture changed.
	 * The callback is called on the game thread. Note that the change may occur on the rendering thread.
	 */
	FMediaCaptureStateChangedSignatureNative OnStateChangedNative;

	/**
	 * Returns true if media capture implementation supports synchronization logic. If so,
	 * OnOutputSynchronization delegate below must be called before each captured frame is exported.
	 */
	virtual bool IsOutputSynchronizationSupported() const { return false; }

	/**
	 * Called when output data is ready to be sent out. This blocking call allows to
	 * postpone data export based on the synchronization logic.
	 *
	 * This delegate must be called if IsOutputSynchronizationSupported() returns true.
	 */
	FMediaCaptureOutputSynchronization OnOutputSynchronization;

	UE_API void SetFixedViewportSize(TSharedPtr<FSceneViewport> InSceneViewport, FIntPoint InSize);
	UE_API void ResetFixedViewportSize(TSharedPtr<FSceneViewport> InViewport, bool bInFlushRenderingCommands);

	/** Called after initialize for viewport capture type */
	virtual bool PostInitializeCaptureViewport(TSharedPtr<FSceneViewport>& InSceneViewport) { return true; }

	/** Called after initialize for render target capture type */
	virtual bool PostInitializeCaptureRenderTarget(UTextureRenderTarget2D* InRenderTarget) { return true; }

	/** Called after initialize for rhi resource capture type */
	virtual bool PostInitializeCaptureRHIResource(const FRHICaptureResourceDescription& InResourceDescription) { return true; }

	/** Called after initialize for media texture capture type */
	virtual bool PostInitializeCaptureMediaTexture(UMediaTexture* InMediaTexture) { return true; }
	
	virtual bool UpdateSceneViewportImpl(TSharedPtr<FSceneViewport>& InSceneViewport) { return true; }
	virtual bool UpdateRenderTargetImpl(UTextureRenderTarget2D* InRenderTarget) { return true; }
	virtual bool UpdateMediaTextureImpl(UMediaTexture* InMediaTexture) { return true; }
	virtual bool UpdateAudioDeviceImpl(const FAudioDeviceHandle& InAudioDeviceHandle) { return true; }

	/**
	 * Get the RGB to YUV Conversion Matrix, should be overriden for media outputs that support different color gamuts (ie. BT 2020).
	 * Will default to returning the RGBToYUV Rec709 matrix.
	 */
	UE_API virtual const FMatrix& GetRGBToYUVConversionMatrix() const;

	static UE_API const TSet<EPixelFormat>& GetSupportedRgbaSwizzleFormats();

	/** Get the name of the media output that created this capture. */
	FString GetMediaOutputName() const
	{
		return MediaOutputName;
	}

public:
	//~ UObject interface
	UE_API virtual void BeginDestroy() override;
	UE_API virtual FString GetDesc() override;

protected:
	/** Implementations can return false when the output sdk is not ready. When this function returns false, the current output frame will be discarded. */
	virtual bool IsReadyToOutput() const { return true; }

	//~ UMediaCapture interface
	UE_API virtual bool ValidateMediaOutput() const;

	UE_DEPRECATED(5.1, "This method has been deprecated. Please override InitializeCapture() and PostInitializeCapture variants if needed instead.")
	UE_API virtual bool CaptureSceneViewportImpl(TSharedPtr<FSceneViewport>& InSceneViewport) final;

	UE_DEPRECATED(5.1, "This method has been deprecated. Please override InitializeCapture() and PostInitializeCapture variants if needed instead.")
	UE_API virtual bool CaptureRenderTargetImpl(UTextureRenderTarget2D* InRenderTarget) final;

	/** Initialization method to prepare implementation for capture */
	virtual bool InitializeCapture() PURE_VIRTUAL(UMediaCapture::InitializeCapture, return false; );
	
	virtual void StopCaptureImpl(bool bAllowPendingFrameToBeProcess) { }

	/** Returns whether the capture implementation supports the implementation of StopCapture that returns a future.*/
	virtual bool SupportsStopCapture_Future() const { return false; }

	/**
	 * Version of StopCaptureImpl that returns a future.
	 * Override this in your capture implementation to allow MediaCapture to wait on the StopCapture to complete.
	 * @note: Must be called from Game thread.
	 */
	virtual TFuture<void> StopCapture_Future()
	{
		ensureMsgf(false, TEXT("Attempted to call StopCapture_Future without checking if the implementation supports it via SupportsStopCapture_Future."));
		return MakeFulfilledPromise<void>().GetFuture();
	}

	/**
	 * Override this to cleanup resources in the capture implementation after StopCapture is done. 
	 * @note: Will be invoked from the game thread.
	 */
	virtual void PostCaptureCleanupImpl()
	{
	}

	//~ DMA Functions implemented in child classes, Temporary until AJA and BM middleman are migrated to a module.
	virtual void LockDMATexture_RenderThread(FTextureRHIRef InTexture) {}
	virtual void UnlockDMATexture_RenderThread(FTextureRHIRef InTexture) {}

	/** Whether the capture callbacks can be called on any thread. If true, the _AnyThread callbacks will be used instead of the _RenderThread ones. */
	virtual bool SupportsAnyThreadCapture() const
	{
		return false;
	}

	/** Whether the capture implementation supports restarting on input format change. */
	virtual bool SupportsAutoRestart() const
	{
		return false;
	}

	friend class UE::MediaCaptureData::FCaptureFrame;
	friend class UE::MediaCaptureData::FMediaCaptureHelper;
	friend class UE::MediaCaptureData::FSyncPointWatcher;
	struct FCaptureBaseData
	{
		FTimecode SourceFrameTimecode;
		FFrameRate SourceFrameTimecodeFramerate;
		uint64 SourceFrameNumberRenderThread = 0;
		uint64 SourceFrameNumber = 0;
	};

	

	/**
	 * Capture the data that will pass along to the callback.
	 * @note The capture is done on the Render Thread but triggered from the Game Thread.
	 */
	virtual TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> GetCaptureFrameUserData_GameThread() { return TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe>(); }

	UE_DEPRECATED(5.1, "ShouldCaptureRHITexture has been deprecated to support both texture and buffer output resource. Please use the ShouldCaptureRHIResource instead.")
	virtual bool ShouldCaptureRHITexture() const { return false; }
	
	/** Should we call OnFrameCaptured_RenderingThread() with a RHI resource -or- readback the memory to CPU ram and call OnFrameCaptured_RenderingThread(). */
	virtual bool ShouldCaptureRHIResource() const { return false; }

	/** Allows capture implementaion to decide whether to discard a frame based on its base data. */ 
	virtual bool ShouldCaptureThisFrame(const FCaptureBaseData& InBaseData) const { return true; }

	/** Called at the beginning of the Capture_RenderThread call if output resource type is Texture */
	virtual void BeforeFrameCaptured_RenderingThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, FTextureRHIRef InTexture) {}

	/** Called at the beginning of the Capture_RenderThread call if output resource type is Buffer */
	virtual void BeforeFrameCaptured_RenderingThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, FBufferRHIRef InBuffer) {}

	/**
	 * Callback when the buffer was successfully copied to CPU ram.
	 * The callback in called from a critical point. If you intend to process the buffer, do so in another thread.
	 * The buffer is only valid for the duration of the callback.
	 */
	virtual void OnFrameCaptured_RenderingThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, void* InBuffer, int32 Width, int32 Height, int32 BytesPerRow) { }

	/**
	 * Args passed to OnFrameCaptured
	 **/
	struct FMediaCaptureResourceData
	{
		void* Buffer = nullptr;
		int32 Width = 0;
		int32 Height = 0;
		int32 BytesPerRow = 0;
	};

	/**
	 * Callback when the buffer was successfully copied to CPU ram.
	 * The buffer is only valid for the duration of the callback.
	 * @Note SupportsAnyThreadCapture must return true in the implementation in order for this to be called.
	 */
	virtual void OnFrameCaptured_AnyThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, const FMediaCaptureResourceData& InResourceData) { }

	UE_DEPRECATED(5.1, "OnRHITextureCaptured_RenderingThread has been deprecated to support texture and buffer output resource. Please use OnRHIResourceCaptured_RenderingThread instead.")
	virtual void OnRHITextureCaptured_RenderingThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, FTextureRHIRef InTexture) { }

	/**
	 * Callbacks when the buffer was successfully copied on the GPU ram.
	 * The callback in called from a critical point. If you intend to process the texture/buffer, do so in another thread.
	 * The texture is valid for the duration of the callback.
	 */
	virtual void OnRHIResourceCaptured_RenderingThread(FRHICommandListImmediate& RHICmdList, const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, FTextureRHIRef InTexture) { }
	virtual void OnRHIResourceCaptured_RenderingThread(FRHICommandListImmediate& RHICmdList, const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, FBufferRHIRef InBuffer) { }
	
	/**
	 * AnyThread version of the above callbacks.
	 * @Note SupportsAnyThreadCapture must return true in the implementation in order for this to be called.
	 */
	virtual void OnRHIResourceCaptured_AnyThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, FTextureRHIRef InTexture) { }
	virtual void OnRHIResourceCaptured_AnyThread(const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, FBufferRHIRef InBuffer) { }

	/**
	 * Callback when the source texture is ready to be copied on the GPU. You need to copy yourself. You may use the callback to add passes to the graph builder.
	 * Graph execution and output texture readback will be handled after this call. No need to do it in the override.
	 * The callback in called from a critical point be fast.
	 * The texture is valid for the duration of the callback.
	 * Only called when the conversion operation is custom and the output resource type is Texture.
	 */
	virtual void OnCustomCapture_RenderingThread(FRDGBuilder& GraphBuilder, const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, FRDGTextureRef InSourceTexture, FRDGTextureRef OutputTexture, const FRHICopyTextureInfo& CopyInfo, FVector2D CropU, FVector2D CropV)
	{
	}

	/**
	 * Callback when the source texture is ready to be copied on the GPU. You need to copy yourself. You may use the callback to add passes to the graph builder.
	 * Graph execution and output buffer readback will be handled after this call. No need to do it in the override.
	 * The callback in called from a critical point be fast.
	 * The texture is valid for the duration of the callback.
	 * Only called when the conversion operation is custom and the output resource type is Buffer.
	 */
	virtual void OnCustomCapture_RenderingThread(FRDGBuilder& GraphBuilder, const FCaptureBaseData& InBaseData, TSharedPtr<FMediaCaptureUserData, ESPMode::ThreadSafe> InUserData, FRDGTextureRef InSourceTexture, FRDGBufferRef OutputBuffer, const FRHICopyTextureInfo& CopyInfo, FVector2D CropU, FVector2D CropV)
	{
	}

	/** Get the size of the buffer when the conversion operation is custom. The function is called once at initialization and the result is cached. */
	virtual FIntPoint GetCustomOutputSize(const FIntPoint& InSize) const 
	{
		return InSize; 
	}

	/** Get the pixel format of the buffer when the conversion operation is custom. The function is called once at initialization and the result is cached. */
	virtual EPixelFormat GetCustomOutputPixelFormat(const EPixelFormat& InPixelFormat) const
	{
		return InPixelFormat;
	}
	
	/** Get the desired output resource type when the conversion is custom. This function is called once at initialization and the result is cached */
	virtual EMediaCaptureResourceType GetCustomOutputResourceType() const 
	{
		return DesiredOutputResourceType; 
	}

	/** Get the desired output buffer description when the conversion is custom and operating in buffer resource type */
	virtual FRDGBufferDesc GetCustomBufferDescription(const FIntPoint& InDesiredSize) const
	{
		return DesiredOutputBufferDescription;
	}

	/** Get the output texture's flags.
     * This can overriden to specify custom texture flags for the output texture.
     * This method can be overriden when doing a custom capture pass; however, this is advanced usage so proceed with caution when changing these.
     */
    virtual ETextureCreateFlags GetOutputTextureFlags() const
    {
        return TexCreate_Shared | TexCreate_RenderTargetable | TexCreate_UAV;
    }

	/** When using GPUTextureTransfer, calling this will wait for a signal set by the underlying library. */
	virtual void WaitForGPU(FRHITexture* InRHITexture) {};

protected:
	UE_API UTextureRenderTarget2D* GetTextureRenderTarget() const;
	UE_API TSharedPtr<FSceneViewport> GetCapturingSceneViewport() const;
	UE_API FString GetCaptureSourceType() const;
	UE_API void SetState(EMediaCaptureState InNewState);
	UE_API void RestartCapture();
	EMediaCaptureConversionOperation GetConversionOperation() const { return ConversionOperation; }

	/** 
	 * Whether capture should be done using experimental scheduling.
	 * Experimental scheduling means that readback is started on the render thread but we don't block on the game thread if it hasn't been completed by the next frame.
	 */
	UE_API bool UseExperimentalScheduling() const;

#if WITH_EDITOR
	/** Gets the prefix to attach to engine analytics events for this media capture */
	UE_API virtual FString GetAnalyticsEventPrefix() const { return TEXT("MediaFramework.MediaCapture"); }
	
	/** Allows media captures to add additional analytics event attributes to an analytics event being recorded */
	UE_API virtual void GetAnalyticsEventAttributes(const FString& EventName, TArray<FAnalyticsEventAttribute>& InOutAttributes) const;
	
	/** Records a new event to engine analytics to indicate the capture has been started */
	UE_API void RecordCaptureStartedEvent() const;
	
	/**
	 * Gets whether the media capture should automatically record a capture started event after InitializeCapture is called.
	 * Media captures that operate asynchronously should set this to false and call RecordCaptureStartedEvent manually when appropriate.
	 */
	UE_API virtual bool ShouldAutoRecordCaptureStartedEvent() const { return true; }
#endif

private:
	void InitializeOutputResources(int32 InNumberOfBuffers);
	
	void OnBeginFrame_GameThread();
	void OnEndFrame_GameThread();
	void OnViewportRendered_GameThread(FViewport* Viewport);
	void CacheMediaOutput(EMediaCaptureSourceType InSourceType);
	void CacheOutputOptions();
	FIntPoint GetOutputSize() const;
	EPixelFormat GetOutputPixelFormat() const;
	EMediaCaptureResourceType GetOutputResourceType() const;
	FRDGBufferDesc GetOutputBufferDescription() const;
	FRDGTextureDesc GetOutputTextureDescription() const;
	void BroadcastStateChanged();
	void WaitForPendingTasks();
	void OnBackBufferReadyToPresent(SWindow& SlateWindow, ISlateViewportProvider& ViewportProvider);
	
	void WaitForSingleExperimentalSchedulingTaskToComplete();
	void WaitForAllExperimentalSchedulingTasksToComplete();
	
	bool CaptureImmediate_RenderThread(const UE::MediaCaptureData::FCaptureFrameArgs& Args);
	// Capture pipeline stuff
	void ProcessCapture_GameThread();
	void PrepareAndDispatchCapture_GameThread(const TSharedPtr<UE::MediaCaptureData::FCaptureFrame>& CapturingFrame);
	
	bool ProcessCapture_RenderThread(const TSharedPtr<UE::MediaCaptureData::FCaptureFrame>& CapturingFrame, UE::MediaCaptureData::FCaptureFrameArgs Args);
	bool ProcessReadyFrame_RenderThread(FRHICommandListImmediate& RHICmdList, UMediaCapture* InMediaCapture, const TSharedPtr<UE::MediaCaptureData::FCaptureFrame>& ReadyFrame);
	
	void InitializeSyncHandlers_RenderThread();
	void InitializeCaptureFrame(const TSharedPtr<UE::MediaCaptureData::FCaptureFrame>& CaptureFrame);

	struct FMediaCaptureSyncData;
	/** Return a sync handler if there is a free one available. */
	TSharedPtr<FMediaCaptureSyncData> GetAvailableSyncHandler() const;

	/** Print the state (Ready/Capturing) of all capture frames. */
	void PrintFrameState();

	/** Common path used by the different capture methods (Scene viewport, render target) to start doing a media capture. */
	bool StartSourceCapture(TSharedPtr<UE::MediaCapture::Private::FCaptureSource> InSource);

	/** Common path used by the different capture methods (Scene viewport, render target) to update the capture source. */
	bool UpdateSource(TSharedPtr<UE::MediaCapture::Private::FCaptureSource> InCaptureSource);

	/**
	 * When using anythread capture, the capture callback will be called directly after readback is complete, meaning there will be minimal delay.
	 * When not using it, the capture callback will be called on the render thread which can introduce a delay.
	 */
	bool UseAnyThreadCapture() const;

	/** Caches game thread dependent frame data for use by Capture Immediate. */
	void AddFrameInfoForCaptureImmediate(const uint64 FrameNumber);

	/** Returns bytes per pixel based on pixel format */
	static int32 GetBytesPerPixel(EPixelFormat InPixelFormat);

	/** Cleanup config and resources after StopCaptureImpl is invoked. */
	void PostCaptureCleanup();

protected:

	/** MediaOutput associated with this capture */
	UPROPERTY(Transient)
	TObjectPtr<UMediaOutput> MediaOutput;

	/** Audio device to be captured. If none, main engine's audio device is captured. */
	FAudioDeviceHandle AudioDeviceHandle;
	
	/** Output size of the media capture. If a conversion is done, this resolution might be different than source resolution. i.e. 1080p RGB might be 960x1080 in YUV */
	FIntPoint DesiredOutputSize = FIntPoint(1920, 1080);
	
	/** Valid source gpu mask of the source. */
	std::atomic<FRHIGPUMask> ValidSourceGPUMask;

	/** Type of capture currently being done. i.e Viewport, RT, RHI Resource */
	EMediaCaptureSourceType CaptureSourceType = EMediaCaptureSourceType::SCENE_VIEWPORT;

	int32 NumberOfCaptureFrame = 2;

private:	
	TPimplPtr<UE::MediaCaptureData::FFrameManager> FrameManager;
	int32 CaptureRequestCount = 0;
	std::atomic<EMediaCaptureState> MediaState = EMediaCaptureState::Stopped;

	FCriticalSection AccessingCapturingSource;

	FIntPoint DesiredSize = FIntPoint(1920, 1080);
	EPixelFormat DesiredPixelFormat = EPixelFormat::PF_A2B10G10R10;
	EPixelFormat DesiredOutputPixelFormat = EPixelFormat::PF_A2B10G10R10;
	/** The TextureDesc used to create the final output texture that will be readback to the output. */
	FRDGTextureDesc DesiredOutputTextureDescription;
	/** The BufferDesc used to create the final output buffer that will be readback to the output. */
	FRDGBufferDesc DesiredOutputBufferDescription;
	FMediaCaptureOptions DesiredCaptureOptions;
	EMediaCaptureConversionOperation ConversionOperation = EMediaCaptureConversionOperation::NONE;
	EMediaCaptureResourceType DesiredOutputResourceType = EMediaCaptureResourceType::Texture;
	FString MediaOutputName = FString(TEXT("[undefined]"));
	bool bUseRequestedTargetSize = false;
	/** Used to avoid repeatedly logging while the underlying sdk is not ready to output. */
	bool bLoggedOutputNotReady = false;

	/** This indicates the capture is in the process of stopping and is waiting for StopCapture_Future's result to be ready. */
	bool bWaitingForStopCapture = false;

	bool bViewportHasFixedViewportSize = false;
	std::atomic<bool> bPendingOutputResourcesInitialized;
	std::atomic<bool> bSyncHandlersInitialized;
	std::atomic<bool> bShouldCaptureRHIResource;
	std::atomic<int32> WaitingForRenderCommandExecutionCounter;
	TArray<TFuture<void>> PendingCommands;
	std::atomic<int32> PendingFrameCount;
	std::atomic<bool> bIsAutoRestartRequired;

	struct FQueuedCaptureData
	{
		FCaptureBaseData BaseData;
		TSharedPtr<FMediaCaptureUserData> UserData;
	};

	/** Used to synchronize access to the capture data queue. */
	FCriticalSection CaptureDataQueueCriticalSection;
	TArray<FQueuedCaptureData> CaptureDataQueue;
	static constexpr int32 MaxCaptureDataAgeInFrames = 20;

	/** Structure holding data used for synchronization of buffer output */
	friend struct UE::MediaCaptureData::FPendingCaptureData;
	struct FMediaCaptureSyncData
	{
		/**
		 * We use a RHIFence to write to in a pass following the buffer conversion one.
		 * We spawn a task that will poll for the fence to be over (Waiting is not exposed)
		 * and then we will push the captured buffer to the capture callback
		 */
		FGPUFenceRHIRef RHIFence;

		/** Whether fence is currently waiting to be cleared */
		std::atomic<bool> bIsBusy = false;
	};

	/** Array of sync handlers (fence) to sync when captured buffer is completed */
	TArray<TSharedPtr<FMediaCaptureSyncData>> SyncHandlers;

	/** Watcher thread looking after end of capturing frames */
	TPimplPtr<UE::MediaCaptureData::FSyncPointWatcher> SyncPointWatcher;

	/** Capture source */
	TSharedPtr<UE::MediaCapture::Private::FCaptureSource, ESPMode::ThreadSafe> CaptureSource;

	/** Holds a view extension that callbacks  */
	TSharedPtr<class FMediaCaptureSceneViewExtension> ViewExtension;

	/** Holds the render passes that a texture will go through during media capture. */
	TPimplPtr<UE::MediaCapture::FRenderPipeline> CaptureRenderPipeline;

	friend struct UE::MediaCapture::FRenderPass;
	friend class UE::MediaCaptureData::FTextureCaptureFrame;
	friend class UE::MediaCaptureData::FBufferCaptureFrame;
};


template<> struct TStructOpsTypeTraits<FMediaCaptureOptions> : public TStructOpsTypeTraitsBase2<FMediaCaptureOptions>
{
	enum
	{
		WithPostSerialize = true
	};
};

#undef UE_API
