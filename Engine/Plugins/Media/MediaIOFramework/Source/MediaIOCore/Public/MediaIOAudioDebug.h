// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/Engine.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "Logging/LogVerbosity.h"
#include "Misc/Paths.h"

/**
 * Single header drop in file to help diagnose audio issues
 * 
 * Usage:
 * 1. Copy and paste this file where needed.
 * 2. Gather a single channel from the audio data into an array
 * 3. Add MEDIA_IO_DUMP_AUDIO(YourArray)
 * 4. At runtime in the console command, enter MediaIO.DumpAudio <number_of_seconds>
 * 
 * This will generate a .csv and .wav file in your project's saved folder under the "AudioDumps" folder.
 * 
 * Current limitations: Only handles mono input at the moment, assumes 48000 sample rate.
 */

DEFINE_LOG_CATEGORY_STATIC(LogMediaIOAudioDebug, Log, All)

#define MEDIA_IO_DUMP_AUDIO_MONO(Array)\
	UE::MediaIOAudioDebug::GetSingleton().ProcessAudio(TEXT(#Array), Array);

#define MEDIA_IO_DUMP_AUDIO(Name, Buffer, BufferSize, SampleSize, NumChannels)\
	UE::MediaIOAudioDebug::GetSingleton().ProcessAudio(Name, Buffer, BufferSize, SampleSize, NumChannels);

// Note: This was copied from audio.cpp and adapted to support multiple audio bit depth and floating point data.
// This will eventually be removed when these changes are ported to the Audio.cpp file.
namespace UE::MediaIOAudioDebug::Private
{
	static void WriteUInt32ToByteArrayLE(TArray<uint8>& InByteArray, int32& Index, const uint32 Value)
	{
		InByteArray[Index++] = (uint8)(Value >> 0);
		InByteArray[Index++] = (uint8)(Value >> 8);
		InByteArray[Index++] = (uint8)(Value >> 16);
		InByteArray[Index++] = (uint8)(Value >> 24);
	}

	static void WriteUInt16ToByteArrayLE(TArray<uint8>& InByteArray, int32& Index, const uint16 Value)
	{
		InByteArray[Index++] = (uint8)(Value >> 0);
		InByteArray[Index++] = (uint8)(Value >> 8);
	}

	static void SerializeWaveFile(TArray<uint8>& OutWaveFileData, const uint8* InPCMData, const int32 NumBytes, const int32 NumChannels, const int32 SampleRate, const int32 BytesPerSample, bool bFloatingPoint)
	{
		// Reserve space for the raw wave data
		OutWaveFileData.Empty(NumBytes + 44);
		OutWaveFileData.AddZeroed(NumBytes + 44);

		int32 WaveDataByteIndex = 0;

		// Wave Format Serialization ----------

		// FieldName: ChunkID
		// FieldSize: 4 bytes
		// FieldValue: RIFF (FourCC value, big-endian)
		OutWaveFileData[WaveDataByteIndex++] = 'R';
		OutWaveFileData[WaveDataByteIndex++] = 'I';
		OutWaveFileData[WaveDataByteIndex++] = 'F';
		OutWaveFileData[WaveDataByteIndex++] = 'F';

		// ChunkName: ChunkSize: 4 bytes
		// Value: NumBytes + 36. Size of the rest of the chunk following this number. Size of entire file minus 8 bytes.
		WriteUInt32ToByteArrayLE(OutWaveFileData, WaveDataByteIndex, NumBytes + 36);

		// FieldName: Format
		// FieldSize: 4 bytes
		// FieldValue: "WAVE"  (big-endian)
		OutWaveFileData[WaveDataByteIndex++] = 'W';
		OutWaveFileData[WaveDataByteIndex++] = 'A';
		OutWaveFileData[WaveDataByteIndex++] = 'V';
		OutWaveFileData[WaveDataByteIndex++] = 'E';

		// FieldName: Subchunk1ID
		// FieldSize: 4 bytes
		// FieldValue: "fmt "
		OutWaveFileData[WaveDataByteIndex++] = 'f';
		OutWaveFileData[WaveDataByteIndex++] = 'm';
		OutWaveFileData[WaveDataByteIndex++] = 't';
		OutWaveFileData[WaveDataByteIndex++] = ' ';

		// FieldName: Subchunk1Size
		// FieldSize: 4 bytes
		// FieldValue: 16 for PCM
		WriteUInt32ToByteArrayLE(OutWaveFileData, WaveDataByteIndex, 16);

		// FieldName: AudioFormat
		// FieldSize: 2 bytes
		// FieldValue: 1 for PCM, 3 for floating point
		WriteUInt16ToByteArrayLE(OutWaveFileData, WaveDataByteIndex, bFloatingPoint ? 3 : 1);

		// FieldName: NumChannels
		// FieldSize: 2 bytes
		// FieldValue: 1 for for mono
		WriteUInt16ToByteArrayLE(OutWaveFileData, WaveDataByteIndex, NumChannels);

		// FieldName: SampleRate
		// FieldSize: 4 bytes
		// FieldValue: Passed in sample rate
		WriteUInt32ToByteArrayLE(OutWaveFileData, WaveDataByteIndex, SampleRate);

		// FieldName: ByteRate
		// FieldSize: 4 bytes
		// FieldValue: SampleRate * NumChannels * BitsPerSample/8
		int32 ByteRate = SampleRate * NumChannels * BytesPerSample;
		WriteUInt32ToByteArrayLE(OutWaveFileData, WaveDataByteIndex, ByteRate);

		// FieldName: BlockAlign
		// FieldSize: 2 bytes
		// FieldValue: NumChannels * BitsPerSample/8
		int32 BlockAlign = NumChannels * BytesPerSample;
		WriteUInt16ToByteArrayLE(OutWaveFileData, WaveDataByteIndex, BlockAlign);

		// FieldName: BitsPerSample
		// FieldSize: 2 bytes
		// FieldValue: BitsPerSample (16 bits per sample)
		WriteUInt16ToByteArrayLE(OutWaveFileData, WaveDataByteIndex, BytesPerSample * 8);

		// FieldName: Subchunk2ID
		// FieldSize: 4 bytes
		// FieldValue: "data" (big endian)

		OutWaveFileData[WaveDataByteIndex++] = 'd';
		OutWaveFileData[WaveDataByteIndex++] = 'a';
		OutWaveFileData[WaveDataByteIndex++] = 't';
		OutWaveFileData[WaveDataByteIndex++] = 'a';

		// FieldName: Subchunk2Size
		// FieldSize: 4 bytes
		// FieldValue: number of bytes of the data
		WriteUInt32ToByteArrayLE(OutWaveFileData, WaveDataByteIndex, NumBytes);

		// Copy the raw PCM data to the audio file
		FMemory::Memcpy(&OutWaveFileData[WaveDataByteIndex], InPCMData, NumBytes);
	}
}

namespace UE::MediaIOAudioDebug
{
	static int32 SecondsToDumpFor = 1;
	static bool bProcessAudio = false;

	static FAutoConsoleCommand CAudioDebug(
		TEXT("MediaIO.DumpAudio"),
		TEXT("[seconds (default: 1)] - Number of seconds to dump audio for."),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
			{
				if (Args.Num())
				{
					SecondsToDumpFor = FCString::Atoi(*Args[0]);
				}

				UE_LOGF(LogMediaIOAudioDebug, Display, "Starting to dump audio for %d seconds", SecondsToDumpFor);

				bProcessAudio = true;
			}),
		ECVF_Default);

	struct FAudioDebugDump
	{
		template <typename Type>
		bool Start(const FString& InFileName, int32 InFramesToWrite)
		{
			if (bStarted)
			{
				return true;
			}

			const FString Datetime = FDateTime::Now().ToString();
			SampleSize = sizeof(Type);
			NumFramesToWrite = InFramesToWrite;
			BaseFileName = InFileName;
			FolderPath = FPaths::ProjectSavedDir() / TEXT("AudioDumps") / Datetime;

			bStarted = true;
			return true;
		}


		template <typename Type>
		void WriteToWavFile()
		{
			if (bStarted && CollectedSamples.Num())
			{
				TArray<uint8> WaveFileData;
				Private::SerializeWaveFile(WaveFileData, (uint8*)CollectedSamples.GetData(), CollectedSamples.Num(), 1, 48000, sizeof(Type), TIsFloatingPoint<Type>::Value);

				const FString WavFileName = BaseFileName + TEXT(".wav");

				if (FArchive* WavWriter = IFileManager::Get().CreateFileWriter(*(FolderPath / WavFileName)))
				{
					WavWriter->Serialize((void*)WaveFileData.GetData(), WaveFileData.Num());
					WavWriter->Close();
					delete WavWriter;
				}
				else
				{
					UE_LOGF(LogMediaIOAudioDebug, Error, "Could not write wav file!");
				}
			}
		}

		template <typename Type>
		void WriteToCsvFile()
		{
			if (bStarted && CollectedSamples.Num())
			{
				const FString CsvFileName = BaseFileName + TEXT(".csv");

				if (FArchive* CsvWriter = IFileManager::Get().CreateFileWriter(*(FolderPath / CsvFileName)))
				{
					Type* SamplePtr = (Type*)CollectedSamples.GetData();

					for (int32 Index = 0; Index < NumCollectedSamples; Index++)
					{
						const FString ValueAsString = FString::Printf(TFormatSpecifier<Type>::GetFormatSpecifier(), *SamplePtr);
						CsvWriter->Logf(TEXT("%d, %s"), Index, *ValueAsString);
						SamplePtr++;
					}

					CsvWriter->Close();
					delete CsvWriter;
				}
				else
				{
					UE_LOGF(LogMediaIOAudioDebug, Error, "Could not write wav file!");
				}
			}
		}

		template <typename Type>
		void Stop()
		{
			WriteToWavFile<Type>();
			WriteToCsvFile<Type>();

			FrameWrittenIndex = 0;
			LastWroteIndex = 0;

			bStarted = false;
		}

		template <typename Type>
		void CollectSamples(const TArray<Type>& AudioSamples)
		{
			if (bStarted)
			{
				check(SampleSize == sizeof(Type));

				const int32 NewSamplesSize = AudioSamples.Num() * SampleSize;
				const int32 OldSize = CollectedSamples.Num();
				CollectedSamples.AddZeroed(NewSamplesSize);

				uint8* NewSamplesPosition = CollectedSamples.GetData() + OldSize * sizeof(uint8);
				memcpy(NewSamplesPosition, AudioSamples.GetData(), NewSamplesSize);

				NumCollectedSamples += AudioSamples.Num();

				FrameWrittenIndex++;
				if (FrameWrittenIndex == NumFramesToWrite)
				{
					Stop<Type>();
				}
			}
		}

		/** Collects a single channel. */
		template <typename Type>
		void CollectSamples(const uint8* BufferToWrite, int32 BufferSize, uint8 NumChannels)
		{
			if (bStarted)
			{
				const int32 NewSamplesSize = BufferSize * sizeof(Type);
				const int32 OldSize = CollectedSamples.Num();
				CollectedSamples.AddZeroed(NewSamplesSize);

				uint8* NewSamplesPosition = CollectedSamples.GetData() + OldSize * sizeof(uint8);
				
				for (int32 Index = 0; Index < BufferSize; Index += NumChannels * sizeof(Type))
				{
					for (int32 SampleByteIndex = 0; SampleByteIndex < sizeof(Type); SampleByteIndex++)
					{
						CollectedSamples.Add(BufferToWrite[Index + SampleByteIndex]);
					}
				}
				
				NumCollectedSamples += BufferSize / (sizeof(Type) * NumChannels);

				FrameWrittenIndex++;
				if (FrameWrittenIndex == NumFramesToWrite)
				{
					Stop<Type>();
				}
			}
		}
		

		FString FolderPath;
		FString BaseFileName;
		int32 FrameWrittenIndex = 0;
		int32 LastWroteIndex = 0;
		int32 NumFramesToWrite = 0;
		bool bStarted = false;
	
		uint8 SampleSize = 0;
		TArray<uint8> CollectedSamples;
		int32 NumCollectedSamples = 0;
	};

	struct FAudioDebugDumpSystem
	{
		TMap<FString, FAudioDebugDump> DebugInfoMap;

		template <typename Type>
		void StartDebugDump(const FString& BaseFileName)
		{
			if (DebugInfoMap.Contains(BaseFileName))
			{
				//  Already started
				return;
			}

			FAudioDebugDump& DebugDump = DebugInfoMap.FindOrAdd(BaseFileName);
			if (!DebugDump.bStarted)
			{
				int32 Framerate = 24;
				if (GEngine)
				{
					if (GEngine->bUseFixedFrameRate && GEngine->FixedFrameRate != 0)
					{
						Framerate = FMath::RoundToInt32(GEngine->FixedFrameRate);
					}
				}

				if (Framerate != 0)
				{
					DebugDump.Start<Type>(BaseFileName, SecondsToDumpFor * Framerate);
				}
			}
		}

		template <typename Type>
		void ProcessSamples(const FString& BaseFileName, const TArray<Type>& BufferToWrite)
		{
			if (FAudioDebugDump* DebugDump = DebugInfoMap.Find(BaseFileName))
			{
				DebugDump->CollectSamples(BufferToWrite);

				if (!DebugDump->bStarted)
				{
					UE_LOGF(LogMediaIOAudioDebug, Display, "Finished audio dump for %ls", *DebugDump->BaseFileName);
					DebugInfoMap.Remove(BaseFileName);
				}
			}

			if (DebugInfoMap.Num() == 0)
			{
				bProcessAudio = false;
			}
		};

		template <typename Type>
		void ProcessSamples(const FString& BaseFileName, const uint8* BufferToWrite, int32 BufferSize, uint8 NumChannels)
		{
			if (FAudioDebugDump* DebugDump = DebugInfoMap.Find(BaseFileName))
			{
				DebugDump->CollectSamples<Type>(BufferToWrite, BufferSize, NumChannels);

				if (!DebugDump->bStarted)
				{
					UE_LOGF(LogMediaIOAudioDebug, Display, "Finished audio dump for %ls", *DebugDump->BaseFileName);
					DebugInfoMap.Remove(BaseFileName);
				}
			}

			if (DebugInfoMap.Num() == 0)
			{
				bProcessAudio = false;
			}
		};

		template <typename Type>
		void ProcessAudio(const FString& BaseFileName, const TArray<Type>& BufferToWrite)
		{
			if (bProcessAudio)
			{
				StartDebugDump<Type>(BaseFileName);
				ProcessSamples(BaseFileName, BufferToWrite);
			}
		}
		
		template <typename Type>
        void ProcessAudio(const FString& BaseFileName, const uint8* BufferToWrite, int32 BufferSize, uint8 NumChannels)
        {
        	if (bProcessAudio)
        	{
        		StartDebugDump<Type>(BaseFileName);
        		ProcessSamples<Type>(BaseFileName, BufferToWrite, BufferSize, NumChannels);
        	}
        }
	};

	static FAudioDebugDumpSystem& GetSingleton()
	{
		static FAudioDebugDumpSystem DebugSystem;
		return DebugSystem;
	}
}
