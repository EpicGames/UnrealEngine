// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioDeviceHandle.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/MultithreadedPatching.h"
#include "ISubmixBufferListener.h"
#include "Math/NumericLimits.h"
#include "Misc/FrameRate.h"

#if WITH_MEDIA_IO_AUDIO_DEBUGGING
#include "MediaIOAudioDebug.h"
#endif

class FAudioDevice;

class FAudioDeviceHandle;

DECLARE_DELEGATE_TwoParams(FOnBufferReceived, uint8* Buffer, int32 BufferSize);

class FMediaIOAudioOutput
{
public:
	struct FAudioOptions
	{
		uint32 InNumInputChannels = 0;
		uint32 InNumOutputChannels = 0;
		FFrameRate InTargetFrameRate;
		uint32 InMaxSampleLatency = 0;
		uint32 InOutputSampleRate = 0;
	};
	
	MEDIAIOCORE_API FMediaIOAudioOutput(Audio::FPatchOutputStrongPtr InPatchOutput, const FAudioOptions& InAudioOptions);
	
	template <typename OutputType>
	static TArray<OutputType> ConvertAndUpmixBuffer(TConstArrayView<float> InBuffer, int32 InNumInputChannels, int32 InNumOutputChannels)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MediaIoCoreModule::ConvertAndUpmixBuffer);

		/**
		 * @Note: Audio::DownmixBuffer was not used here because it does not convert data in place,
		 *  this forces us to create an additional buffer to hold the final int32 values. Instead we
		 *  upmix and convert the values in a single pass.
		 */

		 // @Note: We only support upmixing at the moment.

		TArray<OutputType> ConvertedBuffer;
		if (InBuffer.Num())
		{
			const float ChannelRatio = static_cast<float>(InNumOutputChannels) / InNumInputChannels;
			ConvertedBuffer.SetNumZeroed(FMath::CeilToInt32(ChannelRatio * InBuffer.Num()));

			OutputType* ConvertedBufferPtr = ConvertedBuffer.GetData();
			const TSampleConverter<OutputType> SamplerConverter;

			for (int32 Index = 0; Index < InBuffer.Num(); Index += InNumInputChannels)
			{
				// Convert values for each channel
				for (int32 ChannelIndex = 0; ChannelIndex < InNumInputChannels; ++ChannelIndex)
				{
					*(ConvertedBufferPtr + ChannelIndex) = SamplerConverter.ConvertSample(InBuffer[Index + ChannelIndex]);
				}

				ConvertedBufferPtr += InNumOutputChannels;
			}

			const int32 FinalNumSamples = AlignDown(ConvertedBuffer.Num(), 4);
			ConvertedBuffer.SetNum(FinalNumSamples, EAllowShrinking::Yes);

#if WITH_MEDIA_IO_AUDIO_DEBUGGING
			//MEDIA_IO_DUMP_AUDIO(InBuffer.GetData(), InBuffer.Num() * sizeof(float), sizeof(float), NumInputChannels);
			UE::MediaIOAudioDebug::GetSingleton().ProcessAudio<float>(TEXT("InitialBuffer"), (uint8*) InBuffer.GetData(), InBuffer.Num() * sizeof(float), InNumInputChannels);
			UE::MediaIOAudioDebug::GetSingleton().ProcessAudio<OutputType>(TEXT("ConvertedBuffer"), (uint8*) ConvertedBuffer.GetData(), ConvertedBuffer.Num() * sizeof(OutputType), InNumOutputChannels);
#endif /*WITH_MEDIA_IO_AUDIO_DEBUGGING*/
		}

		return ConvertedBuffer;
	}

	/**
	 * Get the audio sample that were accumulated.
	 */
	template <typename OutputType>
	TArray<OutputType> GetAudioSamples() const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FMediaIOAudioOutput::GetAudioSamples);
		
		const Audio::FAlignedFloatBuffer FloatBuffer = GetFloatBuffer(NumSamplesPerFrame);
		return ConvertAndUpmixBuffer<OutputType>(FloatBuffer, NumInputChannels, NumOutputChannels);
	}
	
	template <typename OutputType>
	TArray<OutputType> GetAudioSamples(uint32 NumSamplesToGet) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FMediaIOAudioOutput::GetAudioSamples);
		
		const Audio::FAlignedFloatBuffer FloatBuffer = GetFloatBuffer(NumSamplesToGet);
		return ConvertAndUpmixBuffer<OutputType>(FloatBuffer, NumInputChannels, NumOutputChannels);
	}

	template <typename OutputType>
	TArray<OutputType> GetAllAudioSamples() const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FMediaIOAudioOutput::GetAllAudioSamples);

		const Audio::FAlignedFloatBuffer FloatBuffer = GetFloatBuffer(MaxSampleLatency);
		return ConvertAndUpmixBuffer<OutputType>(FloatBuffer, NumInputChannels, NumOutputChannels);
	}

	UE_DEPRECATED(5.6, "This function is deprecated because it is not implemented. Use UMediaIOCoreSubsystem::OnBufferReceived() instead.")
	FOnBufferReceived& OnBufferReceived()
	{
		return BufferReceivedDelegate;
	}

public:
	// @todo: Depend on frame number to correctly fetch the right amount of frames on framerates like 59.97
	int32 NumSamplesPerFrame = 0;

	/** Number of audio channels on the engine side. */
	int32 NumInputChannels;

	/** Number of audio channels to output. */
	int32 NumOutputChannels;

	FFrameRate TargetFrameRate;

	/** Maximum number of samples to accumulate before they are discarded. */
	uint32 MaxSampleLatency;

	uint32 OutputSampleRate;

private:
	MEDIAIOCORE_API int32 GetAudioBuffer(int32 InNumSamplesToPop, float* OutBuffer) const;
	MEDIAIOCORE_API Audio::FAlignedFloatBuffer GetFloatBuffer(uint32 NumSamplesToGet) const;

	/** Sample conversion helper template. */
	template <typename OutputType>
	struct TSampleConverter
	{
		const double Scale;

		constexpr TSampleConverter()
			: Scale(TNumericLimits<OutputType>::Max())
		{}
		
		OutputType ConvertSample(const float InFloatValue) const
		{
			// @Note: While this conversion is a common approach, it can introduce distortion. 
			// See: https://www.cs.cmu.edu/~rbd/papers/cmj-float-to-int.html
			return static_cast<OutputType>((InFloatValue * Scale) + 0.5);
		}
	};

private:
	/** The buffer accumulating audio samples. */
	Audio::FPatchOutputStrongPtr PatchOutput;

	/** Callback called when a buffer is received. (Not used, deprecated) */
	FOnBufferReceived BufferReceivedDelegate;
};

/** Specialization for float buffers. */
template <>
struct FMediaIOAudioOutput::TSampleConverter<float>
{
	float ConvertSample(const float InFloatValue) const
	{
		return InFloatValue; // no op for float.
	}
};

/**
 * Handles capturing audio samples rendered by the engine and dispatching them to outputs.
 */
class FMediaIOAudioCapture : public ISubmixBufferListener
{
public:
	DECLARE_DELEGATE_TwoParams(FOnAudioCaptured, float* /* data */, int32 /* NumSamples */);

	FMediaIOAudioCapture() = default;

	virtual ~FMediaIOAudioCapture() override;

	//~ Begin ISubmixBufferListener
	virtual void OnNewSubmixBuffer(const USoundSubmix* InOwningSubmix, float* InAudioData, int32 InNumSamples, int32 InNumChannels, const int32 InSampleRate, double InAudioClock) override;
	virtual const FString& GetListenerName() const override;
	//~ End ISubmixBufferListener
	
	UE_DEPRECATED(5.6, "Use the version that takes in the AudioCapturedDelegate.")
	void Initialize(const FAudioDeviceHandle& InAudioDeviceHandle);
	
	/**
	 * Initializes audio capture for the given audio device
	 * The audio capture delegate can't be changed post Initialize.
	 */
	void Initialize(const FAudioDeviceHandle& InAudioDeviceHandle, const FOnAudioCaptured& InAudioCapturedDelegate);

	/** Create an audio output that will receive audio samples. */
	TSharedPtr<FMediaIOAudioOutput> CreateAudioOutput(int32 InNumOutputChannels, FFrameRate InTargetFrameRate, uint32 InMaxSampleLatency, uint32 InOutputSampleRate);

	UE_DEPRECATED(5.6, "This function is not thread safe. AudioCapturedDelegate must be passed as parameter to Initialize instead.")
	FOnAudioCaptured& OnAudioCaptured_RenderThread()
	{
		return AudioCapturedDelegate;
	}

	/** Returns the number of channels from the audio mixer device (engine side). */
	int32 GetNumInputChannels() const
	{
		return NumChannels;
	}

protected:
	void RegisterBufferListener(FAudioDevice* AudioDevice);
	void UnregisterBufferListener(FAudioDevice* AudioDevice);

	void RegisterAudioDevice(const FAudioDeviceHandle& InAudioDeviceHandle);
	void UnregisterAudioDevice();

	/** This must be called in Initialize only, and before calling RegisterBufferListener. */
	void SetAudioCapturedDelegate(const FOnAudioCaptured& InAudioCapturedDelegate);

private:
	/** Audio device Id this buffer listener is registered to. */
	Audio::FDeviceId RegisteredDeviceId = INDEX_NONE;
	
	/** Sample rate on the engine side. */ 
	uint32 SampleRate = 0;

	/** Internal implementation of ISubmixBufferListener. */
	class FSubmixBufferListenerProxy;

	/** We need to proxy the submit buffer listener to be able to unregister it in the destructor. */
	TSharedPtr<ISubmixBufferListener> SubmixBufferListenerProxy;
	
	/** Used to make sure we only accumulate audio from the primary submix. */ 
	FName PrimarySubmixName;

	/** Number of channels on the engine side. */
	int32 NumChannels = 0;

	/** Utility that allows pushing audio samples to multiple outputs. */
	Audio::FPatchSplitter AudioSplitter;

	/** Callback for a audio rendered event. */
	FOnAudioCaptured AudioCapturedDelegate;
};

/**
 * Audio capture that automatically registers to the main engine device.
 * Also, handles automatically registering to the current PIE world's audio device.
 * This audio capture is used by default if no audio device handle is specified when media capture
 * creates the audio output.
 */
class FMainMediaIOAudioCapture : public FMediaIOAudioCapture
{
public:
	FMainMediaIOAudioCapture();
	virtual ~FMainMediaIOAudioCapture() override;

	UE_DEPRECATED(5.6, "Use the version that takes in the AudioCapturedDelegate.")
	void Initialize();

	/** Initializes audio capture for the main audio device */
	void Initialize(const FOnAudioCaptured& InAudioCapturedDelegate);

private:
#if WITH_EDITOR
	void OnPIEStarted(const bool);
	void OnPIEEnded(const bool);
#endif

	void RegisterMainAudioDevice();
	void RegisterCurrentAudioDevice();
};
