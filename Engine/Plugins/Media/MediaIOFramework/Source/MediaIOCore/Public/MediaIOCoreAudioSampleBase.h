// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaAudioSample.h"
#include "MediaObjectPool.h"

#define UE_API MEDIAIOCORE_API

/*
 * Implements a media audio sample.
 */
class FMediaIOCoreAudioSampleBase
	: public IMediaAudioSample
	, public IMediaPoolable
{
public:

	/** Default constructor. */
	UE_API FMediaIOCoreAudioSampleBase();

	/**
	 * Initialize the sample.
	 *
	 * @param InAudioBuffer The audio frame data.
	 * @param InBufferSize The size of the audio buffer.
	 * @param InNumberOfChannels The number of channel of the audio buffer.
	 * @param InSampleRate The sample rate of the audio buffer.
	 * @param InTime The sample time (in the player's own clock).
	 * @param InTimecode The sample timecode if available.
	 */
	UE_API bool Initialize(const int32* InAudioBuffer, uint32 InBufferSize, uint32 InNumberOfChannels, uint32 InSampleRate, FTimespan InTime, const TOptional<FTimecode>& InTimecode);

	/**
	 * Initialize the sample.
	 *
	 * @param InAudioBuffer The audio frame data.
	 * @param InNumberOfChannels The number of channel of the audio buffer.
	 * @param InSampleRate The sample rate of the audio buffer.
	 * @param InTime The sample time (in the player's own clock).
	 * @param InTimecode The sample timecode if available.
	 */
	UE_API bool Initialize(TArray<int32> InAudioBuffer, uint32 InNumberOfChannels, uint32 InSampleRate, FTimespan InTime, const TOptional<FTimecode>& InTimecode);


	/**
	 * Set the sample buffer.
	 *
	 * @param InAudioBuffer The audio frame data.
	 * @param InBufferSize The size of the audio buffer.
	 */
	UE_API bool SetBuffer(const int32* InAudioBuffer, uint32 InBufferSize);

	/**
	 * Set the sample buffer.
	 *
	 * @param InAudioBuffer The audio frame data.
	 */
	UE_API bool SetBuffer(TArray<int32> InAudioBuffer);

	/**
	 * Set the sample properties.
	 *
	 * @param InBufferSize The size of the audio buffer.
	 * @param InNumberOfChannels The number of channel of the audio buffer.
	 * @param InSampleRate The sample rate of the audio buffer.
	 * @param InTime The sample time (in the player's own clock).
	 * @param InTimecode The sample timecode if available.
	 */
	UE_API bool SetProperties(uint32 InBufferSize, uint32 InNumberOfChannels, uint32 InSampleRate, FTimespan InTime, const TOptional<FTimecode>& InTimecode);

	/**
	 * Request an uninitialized sample buffer.
	 * Should be used when the buffer could be filled by something else.
	 * SetProperties should still be called after.
	 *
	 * @param InBufferSize The size of the audio buffer.
	 */
	UE_API virtual void* RequestBuffer(uint32 InBufferSize);

public:

	//~ IMediaAudioSample interface

	virtual const void* GetBuffer() override
	{
		return Buffer.GetData();
	}

	virtual uint32 GetChannels() const override
	{
		return Channels;
	}

	virtual FTimespan GetDuration() const override
	{
		return Duration;
	}

	virtual EMediaAudioSampleFormat GetFormat() const override
	{
		return EMediaAudioSampleFormat::Int32;
	}

	virtual uint32 GetFrames() const override
	{
		return Buffer.Num() / Channels;
	}

	virtual uint32 GetSampleRate() const override
	{
		return SampleRate;
	}

	virtual FMediaTimeStamp GetTime() const override
	{
		return FMediaTimeStamp(Time);
	}

	virtual TOptional<FTimecode> GetTimecode() const override
	{
		return Timecode;
	}

public:

	//~ IMediaPoolable interface

	virtual void ShutdownPoolable() override
	{
		FreeSample();
	}

protected:
	virtual void FreeSample()
	{
		Buffer.Reset();
	}

protected:
	/** The sample's frame buffer. */
	TArray<int32> Buffer;

	/** Number of audio channels. */
	uint32 Channels;

	/** The duration for which the sample is valid. */
	FTimespan Duration;

	/** Audio sample rate (in samples per second). */
	uint32 SampleRate;

	/** Sample time. */
	FTimespan Time;

	/** Sample timecode. */
	TOptional<FTimecode> Timecode;
};

#undef UE_API
