// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaBinarySample.h"
#include "MediaObjectPool.h"

#define UE_API MEDIAIOCORE_API

/**
 * Implements a media binary data sample.
 */
class FMediaIOCoreBinarySampleBase
	: public IMediaBinarySample
	, public IMediaPoolable
{
public:

	/** Default constructor. */
	UE_API FMediaIOCoreBinarySampleBase();

	/**
	 * Initialize the sample.
	 *
	 * @param InBinaryBuffer The metadata frame data.
	 * @param InBufferSize The size of the InBinaryBuffer.
	 * @param InTime The sample time (in the player's own clock).
	 * @param InFrameRate The framerate of the media that produce the sample.
	 * @param InTimecode The sample timecode if available.
	 */
	UE_API bool Initialize(const uint8* InBinaryBuffer, uint32 InBufferSize, FTimespan InTime, const FFrameRate& InFrameRate, const TOptional<FTimecode>& InTimecode);

	/**
	 * Initialize the sample.
	 *
	 * @param InBinaryBuffer The metadata frame data.
	 * @param InTimecode The sample timecode if available.
	 * @param InTime The sample time (in the player's own clock).
	 */
	UE_API bool Initialize(TArray<uint8> InBinaryBuffer, FTimespan InTime, const FFrameRate& InFrameRate, const TOptional<FTimecode>& InTimecode);

	/**
	 * Set the sample buffer.
	 *
	 * @param InBinaryBuffer The metadata frame data.
	 * @param InBufferSize The size of the InBinaryBuffer.
	 */
	UE_API bool SetBuffer(const uint8* InBinaryBuffer, uint32 InBufferSize);

	/**
	 * Set the sample buffer.
	 *
	 * @param InBinaryBuffer The metadata frame data.
	 */
	UE_API bool SetBuffer(TArray<uint8> InBinaryBuffer);

	/**
	 * Set the sample properties.
	 *
	 * @param InTime The sample time (in the player's own clock).
	 * @param InFrameRate The framerate of the media that produce the sample.
	 * @param InTimecode The sample timecode if available.
	 */
	UE_API bool SetProperties(FTimespan InTime, const FFrameRate& InFrameRate, const TOptional<FTimecode>& InTimecode);

	/**
	 * Request an uninitialized sample buffer.
	 * Should be used when the buffer could be filled by something else.
	 * SetProperties should still be called after.
	 *
	 * @param InBufferSize The size of the metadata buffer.
	 */
	UE_API virtual void* RequestBuffer(uint32 InBufferSize);

public:

	//~ IMediaBinarySample interface

	virtual const void* GetData() override
	{
		return Buffer.GetData();
	}

	virtual FTimespan GetDuration() const override
	{
		return Duration;
	}

	virtual uint32 GetSize() const override
	{
		return Buffer.Num();
	}

	virtual FMediaTimeStamp GetTime() const override
	{
		return FMediaTimeStamp(Time);
	}

	virtual TOptional<FTimecode> GetTimecode() const override
	{
		return Timecode;
	}

public:

	//~ IMediaPoolable interface

	virtual void ShutdownPoolable() override
	{
		FreeSample();
	}

protected:

	virtual void FreeSample()
	{
		Buffer.Reset();
	}

protected:
	/** The sample's frame buffer. */
	TArray<uint8> Buffer;

	/** Duration for which the sample is valid. */
	FTimespan Duration;

	/** Sample time. */
	FTimespan Time;

	/** Sample timecode. */
	TOptional<FTimecode> Timecode;
};

#undef UE_API
