// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaOverlaySample.h"
#include "MediaObjectPool.h"


/**
 * Implements a media caption sample.
 */
class FMediaIOCoreCaptionSampleBase
	: public IMediaOverlaySample
	, public IMediaPoolable
{
};
