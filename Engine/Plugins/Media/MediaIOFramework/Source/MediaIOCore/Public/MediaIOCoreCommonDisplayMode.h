// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/ArrayView.h"
#include "MediaIOCoreDefinitions.h"
#include "Misc/FrameRate.h"

#define UE_API MEDIAIOCORE_API

struct FMediaIOCommonDisplayModeInfo
{
	int32 Width;
	int32 Height;
	FFrameRate FrameRate;
	EMediaIOStandardType Standard;
	FText Name;
};

struct FMediaIOCommonDisplayModeResolutionInfo
{
	int32 Width;
	int32 Height;
	FText Name;
};

struct FMediaIOCommonDisplayModes
{
	static UE_API TArrayView<const FMediaIOCommonDisplayModeInfo> GetAllModes();
	static UE_API TArrayView<const FMediaIOCommonDisplayModeResolutionInfo> GetAllResolutions();

	static UE_API const FMediaIOCommonDisplayModeResolutionInfo* Find(int32 InWidth, int32 InHeight);
	static UE_API const FMediaIOCommonDisplayModeInfo* Find(int32 InWidth, int32 InHeight, const FFrameRate& InFrameRate, EMediaIOStandardType InStandard);
	static bool Contains(int32 InWidth, int32 InHeight) { return Find(InWidth, InHeight) != nullptr; }
	static bool Contains(int32 InWidth, int32 InHeight, const FFrameRate& InFrameRate, EMediaIOStandardType InStandard) { return Find(InWidth, InHeight, InFrameRate, InStandard) != nullptr; }

	static UE_API FText GetMediaIOCommonDisplayModeResolutionInfoName(int32 InWidth, int32 InHeight);
	static UE_API FText GetMediaIOCommonDisplayModeInfoName(int32 InWidth, int32 InHeight, const FFrameRate& InFrameRate, EMediaIOStandardType InStandard);
};

#undef UE_API
