// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/FrameRate.h"
#include "UObject/ObjectMacros.h"
#include "MediaIOCoreDefinitions.generated.h"

#define UE_API MEDIAIOCORE_API


/**
 * Media transport type.
 */
UENUM()
enum class EMediaIOTransportType
{
	SingleLink,
	DualLink,
	QuadLink,
	HDMI,
};


/**
 * Quad link transport type.
 */
UENUM()
enum class EMediaIOQuadLinkTransportType
{
	SquareDivision,
	TwoSampleInterleave,
};


/**
 * SDI transport type.
 */
UENUM()
enum class EMediaIOStandardType
{
	Progressive,
	Interlaced,
	ProgressiveSegmentedFrame,
};

/**
 * Timecode formats.
 */
UENUM()
enum class EMediaIOTimecodeFormat
{
	None,
	LTC,
	VITC,
};

/**
 * Timecode formats.
 */
UENUM()
enum class EMediaIOAutoDetectableTimecodeFormat : uint8
{
	None = 0,
	LTC,
	VITC,
	Auto = 255,
};

/**
 * Which interlace field should be set first in the buffer.
 */
UENUM()
enum class EMediaIOInterlaceFieldOrder : uint8
{
	TopFieldFirst,
	BottomFieldFirst
};

/**
 * Various sample evaluation types available in MediaIO.
 */
UENUM()
enum class EMediaIOSampleEvaluationType : uint8
{
	Latest,
	PlatformTime,
	Timecode,
};

namespace UE::MediaIO
{
	static EMediaIOAutoDetectableTimecodeFormat ToAutoDetectableTimecodeFormat(EMediaIOTimecodeFormat TimecodeFormat)
	{
		switch (TimecodeFormat)
		{
		case EMediaIOTimecodeFormat::None:
			return EMediaIOAutoDetectableTimecodeFormat::None;
		case EMediaIOTimecodeFormat::LTC:
			return EMediaIOAutoDetectableTimecodeFormat::LTC;
		case EMediaIOTimecodeFormat::VITC:
			return EMediaIOAutoDetectableTimecodeFormat::VITC;
		default:
			ensureMsgf(false, TEXT("Unknown timecode format encountered!"));
			return EMediaIOAutoDetectableTimecodeFormat::None;
		}
	}

	static EMediaIOTimecodeFormat FromAutoDetectableTimecodeFormat(EMediaIOAutoDetectableTimecodeFormat TimecodeFormat)
	{
		switch (TimecodeFormat)
		{
		case EMediaIOAutoDetectableTimecodeFormat::LTC:
			return EMediaIOTimecodeFormat::LTC;
		case EMediaIOAutoDetectableTimecodeFormat::VITC:
			return EMediaIOTimecodeFormat::VITC;
		case EMediaIOAutoDetectableTimecodeFormat::None:
			return EMediaIOTimecodeFormat::None;
		default:
			ensureMsgf(false, TEXT("Unknown timecode format encountered!"));
			return EMediaIOTimecodeFormat::None;
		}
	}
}

USTRUCT()
struct FMediaIOAutoDetectableTimecodeFormat_Backup
{
	GENERATED_BODY()

	/** The timecode format if not autodetected. */
	UPROPERTY(EditAnywhere, Category = "Configuration")
	EMediaIOTimecodeFormat TimecodeFormat = EMediaIOTimecodeFormat::None;
	/** Whether the timecode should be autodetected. */
	UPROPERTY(EditAnywhere, Category = "Configuration")
	bool bAutoDetect = false;
};

/**
 * SDI Input type.
 */
UENUM()
enum class EMediaIOInputType
{
	Fill			UMETA(DisplayName="Fill"),
	FillAndKey		UMETA(DisplayName="Fill & Key"),
};

/**
 * SDI Output type.
 */
UENUM()
enum class EMediaIOOutputType
{
	Fill			UMETA(DisplayName="Fill"),
	FillAndKey		UMETA(DisplayName="Fill & Key"),
};


/**
 * SDI Output type.
 */
UENUM()
enum class EMediaIOReferenceType
{
	FreeRun,
	External,
	Input
};


/**
 * Identifies a device.
 */
USTRUCT()
struct FMediaIODevice
{
	GENERATED_BODY()

public:
	UE_API FMediaIODevice();

	/** The retail/display name of the Device. */
	UPROPERTY(EditAnywhere, Category=Device)
	FName DeviceName;

	/** The device identifier. */
	UPROPERTY(EditAnywhere, Category=Device)
	int32 DeviceIdentifier;

public:
	UE_API bool operator==(const FMediaIODevice& Other) const;

	/** Return true if the device has been set properly */
	UE_API bool IsValid() const;
};


/**
 * Identifies an media connection.
 */
USTRUCT()
struct FMediaIOConnection
{
	GENERATED_BODY()

public:
	UE_API FMediaIOConnection();

	/** The device identifier. */
	UPROPERTY(EditAnywhere, Category=Connection)
	FMediaIODevice Device;
	
	/** The protocol used by the MediaFramework. */
	UPROPERTY(EditAnywhere, Category=Connection)
	FName Protocol;

	/** The type of cable link used for that configuration */
	UPROPERTY(EditAnywhere, Category=Connection)
	EMediaIOTransportType TransportType;

	/** The type of cable link used for that configuration */
	UPROPERTY(EditAnywhere, Category=Connection)
	EMediaIOQuadLinkTransportType QuadTransportType;

	/** The port of the video channel on the device. */
	UPROPERTY(EditAnywhere, Category=Connection)
	int32 PortIdentifier;

public:
	UE_API bool operator==(const FMediaIOConnection& Other) const;

	/**
	 * Get a url used by the media framework
	 * @return Url representation, "protocol://device0/single1"
	 */
	UE_API FString ToUrl() const;

	/** Return true if the connection has been set properly */
	UE_API bool IsValid() const;
};


/**
 * Identifies a media mode.
 */
USTRUCT()
struct FMediaIOMode
{
	GENERATED_BODY()

public:
	UE_API FMediaIOMode();

public:
	/** The mode's frame rate. */
	UPROPERTY(VisibleAnywhere, Category=Mode)
	FFrameRate FrameRate;

	/** The mode's image resolution. */
	UPROPERTY(VisibleAnywhere, Category=Mode)
	FIntPoint Resolution;

	/** The mode's scanning type. */
	UPROPERTY(VisibleAnywhere, Category=Mode)
	EMediaIOStandardType Standard;

	/** The mode's identifier for the device. */
	UPROPERTY(VisibleAnywhere, Category=Mode)
	int32 DeviceModeIdentifier;

public:
	UE_API bool operator==(const FMediaIOMode& Other) const;

	/** Return the name of this mode from MediaIOCommonDisplayModes. */
	UE_API FText GetModeName() const;

	/** Return true if the mode has been set properly. */
	UE_API bool IsValid() const;
};


/**
 * Configuration of a device input / output.
 */
USTRUCT()
struct FMediaIOConfiguration
{
	GENERATED_BODY()
	
public:
	UE_API FMediaIOConfiguration();

public:
	/** Configured as an input or output. */
	UPROPERTY()
	bool bIsInput;

	/** The configuration's device and transport type. */
	UPROPERTY(VisibleAnywhere, Category=Configuration)
	FMediaIOConnection MediaConnection;

	/** The configuration's video mode. */
	UPROPERTY(VisibleAnywhere, Category=Configuration)
	FMediaIOMode MediaMode;

public:
	UE_API bool operator== (const FMediaIOConfiguration& Other) const;

	/** Return true if the configuration has been set properly */
	UE_API bool IsValid() const;
};

/**
 * Configuration of a device input.
 */
USTRUCT()
struct FMediaIOInputConfiguration
{
	GENERATED_BODY()

public:
	UE_API FMediaIOInputConfiguration();

	/** The signal input format. */
	UPROPERTY(VisibleAnywhere, Category=Configuration)
	FMediaIOConfiguration MediaConfiguration;

	/** Whether to input the fill or the fill and key. */
	UPROPERTY(VisibleAnywhere, Category=Configuration)
	EMediaIOInputType InputType;

	/** The port of the video channel on the device to input the key from. */
	UPROPERTY(VisibleAnywhere, Category=Configuration)
	int32 KeyPortIdentifier;

public:
	UE_API bool operator== (const FMediaIOInputConfiguration& Other) const;

	/** Return true if the configuration has been set properly */
	UE_API bool IsValid() const;
};

/**
 * Configuration of a device output.
 */
USTRUCT()
struct FMediaIOOutputConfiguration
{
	GENERATED_BODY()

public:
	UE_API FMediaIOOutputConfiguration();

	/** The signal output format. */
	UPROPERTY(VisibleAnywhere, Category=Configuration)
	FMediaIOConfiguration MediaConfiguration;

	/** Whether to output the fill or the fill and key. */
	UPROPERTY(VisibleAnywhere, Category=Configuration)
	EMediaIOOutputType OutputType;

	/**
	 * The port of the video channel on the device to output the key to.
	 * @note	'Frame Buffer Pixel Format' must be set to at least 8 bits of alpha.
	 * @note	'Enable alpha channel support in post-processing' must be set to 'Allow through tonemapper'.
	 */
	UPROPERTY(VisibleAnywhere, Category=Configuration)
	int32 KeyPortIdentifier;

	/** The Device output sync with either its internal clock, an external reference, or an other input. */
	UPROPERTY(VisibleAnywhere, Category=Configuration)
	EMediaIOReferenceType OutputReference;

	/** The port of the video channel on the device to output the synchronize to. */
	UPROPERTY(VisibleAnywhere, Category=Configuration)
	int32 ReferencePortIdentifier;

public:
	UE_API bool operator== (const FMediaIOOutputConfiguration& Other) const;

	/** Return true if the configuration has been set properly */
	UE_API bool IsValid() const;
};

/**
 * Configuration of a Timecode from Video
 */
USTRUCT()
struct FMediaIOVideoTimecodeConfiguration
{
	GENERATED_BODY()
	
	/** Read the timecode from a video signal. */
	UPROPERTY(VisibleAnywhere, Category=Configuration)
	FMediaIOConfiguration MediaConfiguration;

	/** Timecode format to read from a video signal. */
	UPROPERTY(VisibleAnywhere, Category=Configuration)
	EMediaIOAutoDetectableTimecodeFormat TimecodeFormat = EMediaIOAutoDetectableTimecodeFormat::Auto;

public:
	/** Return true if the configuration has been set properly */
	UE_API bool IsValid() const;

	UE_API bool operator== (const FMediaIOVideoTimecodeConfiguration& Other) const;

	/**
	 * Get the configuration text representation.
	 * @return String representation, i.e. "Video/Single1/1080p30fps/LTC".
	 */
	UE_API FText ToText(bool bAutoDetected = false) const;
};

#undef UE_API
