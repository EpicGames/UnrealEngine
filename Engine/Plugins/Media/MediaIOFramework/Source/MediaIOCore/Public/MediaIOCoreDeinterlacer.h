// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ColorManagement/ColorSpace.h"
#include "Containers/Array.h"
#include "MediaIOCoreDefinitions.h"
#include "MediaIOCoreTextureSampleBase.h"
#include "Templates/SharedPointer.h"

#include "MediaIOCoreDeinterlacer.generated.h"

#define UE_API MEDIAIOCORE_API

namespace UE::MediaIOCore
{
	/** Description of a video frame. */
	struct FVideoFrame
	{
		FVideoFrame(const void* InVideoBuffer
		, uint32 InBufferSize
		, uint32 InStride
		, uint32 InWidth
		, uint32 InHeight
		, EMediaTextureSampleFormat InSampleFormat
		, FTimespan InTime
		, const FFrameRate& InFrameRate
		, const TOptional<FTimecode>& InTimecode
		, TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> InSourceColorSettings)
			: VideoBuffer(InVideoBuffer)
			, BufferSize(InBufferSize)
			, Stride(InStride)
			, Width(InWidth)
			, Height(InHeight)
			, SampleFormat(InSampleFormat)
			, Time(InTime)
			, FrameRate(InFrameRate)
			, Timecode(InTimecode)
			, SourceColorSettings(InSourceColorSettings)
		{
		}

		const void* VideoBuffer;
		uint32 BufferSize;
		uint32 Stride;
		uint32 Width;
		uint32 Height;
		EMediaTextureSampleFormat SampleFormat;
		FTimespan Time;
		const FFrameRate& FrameRate;
		const TOptional<FTimecode>& Timecode;
		TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> SourceColorSettings;

		UE_DEPRECATED(5.8, "Deprecated constructor. Please use FNativeMediaSourceColorSettings variant.")
		FVideoFrame(const void* InVideoBuffer
			, uint32 InBufferSize
			, uint32 InStride
			, uint32 InWidth
			, uint32 InHeight
			, EMediaTextureSampleFormat InSampleFormat
			, FTimespan InTime
			, const FFrameRate& InFrameRate
			, const TOptional<FTimecode>& InTimecode
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			, const UE::MediaIOCore::FColorFormatArgs& InColorFormatArgs
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		)
			: VideoBuffer(InVideoBuffer)
			, BufferSize(InBufferSize)
			, Stride(InStride)
			, Width(InWidth)
			, Height(InHeight)
			, SampleFormat(InSampleFormat)
			, Time(InTime)
			, FrameRate(InFrameRate)
			, Timecode(InTimecode)
			, SourceColorSettings(nullptr)
		{
		}
	};

	/**
	 * Handles deinterlacing a video signal.
	 */
	class FDeinterlacer
	{
	public:
		DECLARE_DELEGATE_RetVal(TSharedPtr<FMediaIOCoreTextureSampleBase>, FOnAcquireSample_AnyThread);

		FDeinterlacer() = default;
		FDeinterlacer(FOnAcquireSample_AnyThread InOnAcquireSample, EMediaIOInterlaceFieldOrder InInterlaceFieldOrder) 
			: AcquireSampleDelegate(MoveTemp(InOnAcquireSample))
			, InterlaceFieldOrder(InInterlaceFieldOrder)
		{
		}

		virtual ~FDeinterlacer() = default;
        
		/**
		 * Default implementation which applies no deinterlacing.
		 * @param InVideoFrame The video buffer and its metadata.
		 * @return One or more sample depending on the deinterlacing method.
		 */
		UE_API virtual TArray<TSharedRef<FMediaIOCoreTextureSampleBase>> Deinterlace(const FVideoFrame& InVideoFrame) const;
	protected:
		/** Delegate called to acquire samples. */
		FOnAcquireSample_AnyThread AcquireSampleDelegate;
		/** What order should fields be read in. */
		EMediaIOInterlaceFieldOrder InterlaceFieldOrder = EMediaIOInterlaceFieldOrder::TopFieldFirst; 
	};
	
	/** Line-double each field independently. Two output samples per source frame at full vertical resolution; doubles temporal resolution. */
	class FBobDeinterlacer : public FDeinterlacer
	{
	public:
		FBobDeinterlacer(FOnAcquireSample_AnyThread InOnAcquireSample, EMediaIOInterlaceFieldOrder InInterlaceFieldOrder)
			: FDeinterlacer(MoveTemp(InOnAcquireSample), InInterlaceFieldOrder)
		{
		}
		virtual TArray<TSharedRef<FMediaIOCoreTextureSampleBase>> Deinterlace(const FVideoFrame& InVideoFrame) const override;
	};

	/** Line-doubles both fields to full height and per-pixel averages them. One sample per source frame. */
	class FBlendDeinterlacer : public FDeinterlacer
	{
	public:
		FBlendDeinterlacer(FOnAcquireSample_AnyThread InOnAcquireSample, EMediaIOInterlaceFieldOrder InInterlaceFieldOrder)
			: FDeinterlacer(MoveTemp(InOnAcquireSample), InInterlaceFieldOrder)
		{
		}
		virtual TArray<TSharedRef<FMediaIOCoreTextureSampleBase>> Deinterlace(const FVideoFrame& InVideoFrame) const override;
	};

	/** Discards one of the fields (based on InterlaceFieldOrder) and line-doubles the kept field to full height. Halves temporal resolution; preserves vertical resolution via line-doubling. */
	class FDiscardDeinterlacer : public FDeinterlacer
	{
	public:
		FDiscardDeinterlacer(FOnAcquireSample_AnyThread InOnAcquireSample, EMediaIOInterlaceFieldOrder InInterlaceFieldOrder)
			: FDeinterlacer(MoveTemp(InOnAcquireSample), InInterlaceFieldOrder)
		{
		}
		virtual TArray<TSharedRef<FMediaIOCoreTextureSampleBase>> Deinterlace(const FVideoFrame& InVideoFrame) const override;
	};
	
	
}

/**
 * Represents a deinterlacing algorithm. Will dictate how the incoming video signal is converted from interlace to a progressive signal.
 */
UCLASS(MinimalAPI, Abstract, editinlinenew)
class UVideoDeinterlacer : public UObject
{
	GENERATED_BODY()
public:
	/**
	 * Creates an instance of a FDeinterlacer that represents a deinterlacing algorithm.
	 */
	virtual TSharedPtr<UE::MediaIOCore::FDeinterlacer> Instantiate(UE::MediaIOCore::FDeinterlacer::FOnAcquireSample_AnyThread InAcquireSampleDelegate, EMediaIOInterlaceFieldOrder InInterlaceFieldOrder) const PURE_VIRTUAL(UVideoDeinterlacer::Instantiate, return nullptr; );
};


/** Doubles the frame rate by treating each field as its own moment. Smoothest motion when the engine can sustain the doubled rate. */
UCLASS(MinimalAPI)
class UBobDeinterlacer : public UVideoDeinterlacer
{
	GENERATED_BODY()
public:
	virtual TSharedPtr<UE::MediaIOCore::FDeinterlacer> Instantiate(UE::MediaIOCore::FDeinterlacer::FOnAcquireSample_AnyThread InAcquireSampleDelegate, EMediaIOInterlaceFieldOrder InInterlaceFieldOrder) const override
	{
		return MakeShared<UE::MediaIOCore::FBobDeinterlacer>(MoveTemp(InAcquireSampleDelegate), InInterlaceFieldOrder);
	}
};

/** Combines both fields into a single full-resolution image. Moving objects appear slightly ghosted. */
UCLASS(MinimalAPI)
class UBlendDeinterlacer : public UVideoDeinterlacer
{
	GENERATED_BODY()
public:
	virtual TSharedPtr<UE::MediaIOCore::FDeinterlacer> Instantiate(UE::MediaIOCore::FDeinterlacer::FOnAcquireSample_AnyThread InAcquireSampleDelegate, EMediaIOInterlaceFieldOrder InInterlaceFieldOrder) const override
	{
		return MakeShared<UE::MediaIOCore::FBlendDeinterlacer>(MoveTemp(InAcquireSampleDelegate), InInterlaceFieldOrder);
	}
};

/** Keeps a single field and line-doubles it. Simple and clean, but with reduced vertical detail. */
UCLASS(MinimalAPI)
class UDiscardDeinterlacer : public UVideoDeinterlacer
{
	GENERATED_BODY()
public:
	virtual TSharedPtr<UE::MediaIOCore::FDeinterlacer> Instantiate(UE::MediaIOCore::FDeinterlacer::FOnAcquireSample_AnyThread InAcquireSampleDelegate, EMediaIOInterlaceFieldOrder InInterlaceFieldOrder) const override
	{
		return MakeShared<UE::MediaIOCore::FDiscardDeinterlacer>(MoveTemp(InAcquireSampleDelegate), InInterlaceFieldOrder);
	}
};

#undef UE_API
