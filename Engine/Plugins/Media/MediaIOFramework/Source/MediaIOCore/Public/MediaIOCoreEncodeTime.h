// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IMediaTextureSample.h"

#define UE_API MEDIAIOCORE_API

enum class EMediaIOCoreEncodePixelFormat
{
	A2B10G10R10,
	CharBGRA,
	CharUYVY,
	YUVv210,
};

class FMediaIOCoreEncodeTime
{
public:
	UE_API FMediaIOCoreEncodeTime(EMediaIOCoreEncodePixelFormat InFormat, void* InBuffer, uint32 InPitch, uint32 InWidth, uint32 InHeight, uint32 XOffset = 0, uint32 YOffset = 0);
	UE_API void Render(uint32 InHours, uint32 InMinutes, uint32 InSeconds, uint32 InFrames) const;

protected:
	using TColor = int32;
	UE_API void DrawChar(uint32 InX, uint32 InChar) const;
	UE_API void DrawTime(uint32 InX, uint32 InTime) const;
	UE_API void SetPixelScaled(uint32 InX, uint32 InY, bool InSet, uint32 InScale) const;
	UE_API void SetPixel(uint32 InX, uint32 InY, bool InSet) const;

	inline TColor& At(uint32 InX, uint32 InY) const
	{
		return *(reinterpret_cast<TColor*>(reinterpret_cast<char*>(Buffer) + (Pitch * InY)) + InX);
	}

protected:
	/** Pixel format */
	EMediaIOCoreEncodePixelFormat Format;

	/** Pointer to pixels */
	void* Buffer;

	/** Pitch of image */
	uint32 Pitch;

	/** Width of image */
	uint32 Width;

	/** Height of image */
	uint32 Height;

	/** X pixel offset of the rendered timecode. */
	uint32 XPixelOffset;

	/** Y pixel offset of the rendered timecode. */
	uint32 YPixelOffset;

protected:
	TColor ColorBlack;
	TColor ColorWhite;
};

#undef UE_API
