// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "BaseMediaSourceColorSettings.h"
#include "Engine/TextureDefines.h"
#include "IMediaCache.h"
#include "IMediaControls.h"
#include "IMediaPlayer.h"
#include "IMediaTracks.h"
#include "IMediaView.h"
#include "ITimedDataInput.h"

#include "MediaIOCoreDefinitions.h"
#include "MediaIOCoreSamples.h"
#include "MediaIOCoreSampleContainer.h"


#include "HAL/CriticalSection.h"
#include "Misc/CoreMisc.h"
#include "Misc/FrameRate.h"

#define UE_API MEDIAIOCORE_API

class FMediaIOCoreAudioSampleBase;
class FMediaIOCoreBinarySampleBase;
class FMediaIOCoreCaptionSampleBase;
class FMediaIOCoreSamples;
class FMediaIOCoreSubtitleSampleBase;
class FMediaIOCoreTextureSampleBase;
class FMediaIOCoreTextureSampleConverter;
class IMediaEventSink;

#if WITH_EDITOR
struct FAnalyticsEventAttribute;
#endif

enum class EMediaTextureSampleFormat;

enum class EMediaIOCoreColorFormat : uint8
{
	YUV8,
	YUV10
};

struct FMediaIOCoreMediaOption
{
	static UE_API const FName FrameRateNumerator;
	static UE_API const FName FrameRateDenominator;
	static UE_API const FName ResolutionWidth;
	static UE_API const FName ResolutionHeight;
	static UE_API const FName VideoModeName;
};

namespace UE::GPUTextureTransfer
{
	using TextureTransferPtr = TSharedPtr<class ITextureTransfer>;
}

namespace UE::MediaIOCore
{
	class FDeinterlacer;
	struct FVideoFrame;
}
 
/**
 * Implements a base player for hardware IO cards. 
 *
 * The processing of metadata and video frames is delayed until the fetch stage
 * (TickFetch) in order to increase the window of opportunity for receiving
 * frames for the current render frame time code.
 *
 * Depending on whether the media source enables time code synchronization,
 * the player's current play time (CurrentTime) is derived either from the
 * time codes embedded in frames or from the Engine's global time code.
 */
class FMediaIOCorePlayerBase
	: public IMediaPlayer
	, protected IMediaCache
	, protected IMediaControls
	, protected IMediaTracks
	, protected IMediaView
	, public ITimedDataInput
	, public TSharedFromThis<FMediaIOCorePlayerBase>
{
public:

	/**
	 * Create and initialize a new instance.
	 *
	 * @param InEventSink The object that receives media events from this player.
	 */
	UE_API FMediaIOCorePlayerBase(IMediaEventSink& InEventSink);

	/** Virtual destructor. */
	UE_API virtual ~FMediaIOCorePlayerBase();

public:

	//~ IMediaPlayer interface

	UE_API virtual void Close() override;
	UE_API virtual bool Open(const FString& Url, const IMediaOptions* Options) override;
	UE_API virtual bool Open(const TSharedRef<FArchive, ESPMode::ThreadSafe>& Archive, const FString& OriginalUrl, const IMediaOptions* Options) override;

	UE_API FString GetInfo() const;
	UE_API virtual IMediaCache& GetCache() override;
	UE_API virtual IMediaControls& GetControls() override;
	UE_API virtual IMediaSamples& GetSamples() override;
	UE_API const FMediaIOCoreSamples& GetSamples() const;
	UE_API virtual FString GetStats() const override;
	UE_API virtual IMediaTracks& GetTracks() override;
	UE_API virtual FString GetUrl() const override;
	UE_API virtual IMediaView& GetView() override;
	UE_API virtual void TickTimeManagement();
	UE_API virtual void TickFetch(FTimespan DeltaTime, FTimespan Timecode) override;

#if WITH_EDITOR
	UE_API virtual void RecordSourceOpenedEvent() const override;
	virtual bool ShouldAutoRecordSourceOpenedEvent() const override { return true; }
#endif

public:
	//~ IMediaCache interface
	
	UE_API virtual bool QueryCacheState(EMediaCacheState State, TRangeSet<FTimespan>& OutTimeRanges) const override;
	UE_API virtual int32 GetSampleCount(EMediaCacheState State) const override;

protected:

	//~ IMediaControls interface

	UE_API virtual bool CanControl(EMediaControl Control) const override;
	UE_API virtual FTimespan GetDuration() const override;
	UE_API virtual float GetRate() const override;
	UE_API virtual EMediaState GetState() const override;
	UE_API virtual EMediaStatus GetStatus() const override;
	UE_API virtual TRangeSet<float> GetSupportedRates(EMediaRateThinning Thinning) const override;
	UE_API virtual FTimespan GetTime() const override;
	UE_API virtual bool IsLooping() const override;
	UE_API virtual bool Seek(const FTimespan& Time) override;
	UE_API virtual bool SetLooping(bool Looping) override;
	UE_API virtual bool SetRate(float Rate) override;

protected:

	//~ IMediaTracks interface

	UE_API virtual bool GetAudioTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaAudioTrackFormat& OutFormat) const override;
	UE_API virtual int32 GetNumTracks(EMediaTrackType TrackType) const override;
	UE_API virtual int32 GetNumTrackFormats(EMediaTrackType TrackType, int32 TrackIndex) const override;
	UE_API virtual int32 GetSelectedTrack(EMediaTrackType TrackType) const override;
	UE_API virtual FText GetTrackDisplayName(EMediaTrackType TrackType, int32 TrackIndex) const override;
	UE_API virtual int32 GetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex) const override;
	UE_API virtual FString GetTrackLanguage(EMediaTrackType TrackType, int32 TrackIndex) const override;
	UE_API virtual FString GetTrackName(EMediaTrackType TrackType, int32 TrackIndex) const override;
	UE_API virtual bool GetVideoTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaVideoTrackFormat& OutFormat) const override;
	UE_API virtual bool SelectTrack(EMediaTrackType TrackType, int32 TrackIndex) override;
	UE_API virtual bool SetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex, int32 FormatIndex) override;

public:
	//~ ITimedDataInput interface
	UE_API virtual FText GetDisplayName() const override;
	UE_API virtual TArray<ITimedDataInputChannel*> GetChannels() const override;
	UE_API virtual ETimedDataInputEvaluationType GetEvaluationType() const override;
	UE_API virtual void SetEvaluationType(ETimedDataInputEvaluationType Evaluation) override;
	UE_API virtual double GetEvaluationOffsetInSeconds() const override;
	UE_API virtual void SetEvaluationOffsetInSeconds(double Offset) override;
	UE_API virtual FFrameRate GetFrameRate() const override;
	UE_API virtual bool IsDataBufferSizeControlledByInput() const override;
	UE_API virtual void AddChannel(ITimedDataInputChannel* Channel) override;
	UE_API virtual void RemoveChannel(ITimedDataInputChannel * Channel) override;
	UE_API virtual bool SupportsSubFrames() const override;
	UE_API virtual UObject* GetSettingsObject() const override;

public:
	/** Class used to pass information about current frame for sample picking.*/
	class FFrameInfo
	{
	public:
		FTimecode RequestedTimecode = FTimecode();
		FTimespan SampleTimespan = FTimespan();
		double EvaluationOffset = 0.;
		uint32 FrameNumber = 0.;
	};
	
	/**
	 * Just in time sample rendering. This method is responsible for late sample picking,
	 * then rendering it into the proxy sample provided.
	 */
	UE_API virtual bool JustInTimeSampleRender_RenderThread(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDestinationTexture, TSharedPtr<FMediaIOCoreTextureSampleBase>& JITRProxySample);

protected:

	/** Internal API to push an audio sample to the internal samples pool */
	UE_API void AddAudioSample(const TSharedRef<FMediaIOCoreAudioSampleBase>& Sample);

	/** Internal API to push a caption sample to the internal samples pool */
	UE_API void AddCaptionSample(const TSharedRef<FMediaIOCoreCaptionSampleBase>& Sample);

	/** Internal API to push a metadata sample to the internal samples pool */
	UE_API void AddMetadataSample(const TSharedRef<FMediaIOCoreBinarySampleBase>& Sample);

	/** Internal API to push a subtitle sample to the internal samples pool */
	UE_API void AddSubtitleSample(const TSharedRef<FMediaIOCoreSubtitleSampleBase>& Sample);

	/** Internal API to push a video sample to the internal samples pool */
	UE_API void AddVideoSample(const TSharedRef<FMediaIOCoreTextureSampleBase>& Sample);

protected:
	/** Is the IO hardware/device ready to be used. */
	virtual bool IsHardwareReady() const = 0;

	/** Return true if the options combination are valid. */
	UE_API virtual bool ReadMediaOptions(const IMediaOptions* Options);

	/**
	 * Allows children to notify when video format is known after auto-detection. The main purpose
	 * is to start intialization of the DMA textures at the right moment.
	 */
	UE_API void NotifyVideoFormatDetected();

	/** Get the application time with a delta to represent the actual system time. Use instead of FApp::GetCurrentTime. */
	static UE_API double GetApplicationSeconds();

	/** Get the platform time with a delta to represent the actual system time. Use instead of FApp::Seconds. */
	static UE_API double GetPlatformSeconds();

	/** Log the timecode when a frame is received. */
	static UE_API bool IsTimecodeLogEnabled();
	
	/** 
	 * Setup settings for the different kind of supported data channels. 
	 * PlayerBase will setup the common settings
	 */
	virtual void SetupSampleChannels() = 0;

	/**
	 * Get the number of video frames to buffer.
	 */
	virtual uint32 GetNumVideoFrameBuffers() const
	{ 
		return 1;
	}

	virtual EMediaIOCoreColorFormat GetColorFormat() const 
	{ 
		return EMediaIOCoreColorFormat::YUV8;
	}

	/** Called after fast GPUDirect texture transferring is finished */
	UE_API virtual void AddVideoSampleAfterGPUTransfer_RenderThread(const TSharedRef<FMediaIOCoreTextureSampleBase>& Sample);

	/** Whether fast GPU transferring is available */
	UE_API virtual bool CanUseGPUTextureTransfer() const;

	/** Whether a texture availabled for GPU transfer */
	UE_API bool HasTextureAvailableForGPUTransfer() const;

	/** Whether JITR is available and active */
	UE_API virtual bool IsJustInTimeRenderingEnabled() const;

	/** Factory method to generate texture samples. */
	virtual TSharedPtr<FMediaIOCoreTextureSampleBase> AcquireTextureSample_AnyThread() const = 0;

	/** Deinterlace a video frame. Only valid to call when the video stream is open. */
	UE_API TArray<TSharedRef<FMediaIOCoreTextureSampleBase>> Deinterlace(const UE::MediaIOCore::FVideoFrame& InVideoFrame) const;

protected:

	/** Acquire a proxy sample for JITR. That sample must be initialized for JITR. */
	UE_API virtual TSharedPtr<FMediaIOCoreTextureSampleBase> AcquireJITRProxySampleInitialized();

	/** Factory method to create sample converters. */
	UE_API virtual TSharedPtr<FMediaIOCoreTextureSampleConverter> CreateTextureSampleConverter() const;

	/** Pick a sample to render */
	UE_API virtual TSharedPtr<FMediaIOCoreTextureSampleBase> PickSampleToRender_RenderThread(const FFrameInfo& InFrameInformation);
	UE_API virtual TSharedPtr<FMediaIOCoreTextureSampleBase> PickSampleToRenderForLatest_RenderThread(const FFrameInfo& InFrameInformation);
	UE_API virtual TSharedPtr<FMediaIOCoreTextureSampleBase> PickSampleToRenderForTimeSynchronized_RenderThread(const FFrameInfo& InFrameInformation);
	UE_API virtual TSharedPtr<FMediaIOCoreTextureSampleBase> PickSampleToRenderFramelocked_RenderThread(const FFrameInfo& InFrameInformation);

	/**
	 * A wrapper method responsible for transferring of the sample textures into GPU memory based
	 * on the current settings and hardware capabilities.
	 */
	UE_API void TransferTexture_RenderThread(FRHICommandListImmediate& RHICmdList, const TSharedPtr<FMediaIOCoreTextureSampleBase>& Sample, const TSharedPtr<FMediaIOCoreTextureSampleBase>& JITRProxySample);

#if WITH_EDITOR
	/** Gets the prefix to append to named analytics events recorded by the media player */
	UE_API virtual FString GetAnalyticsEventPrefix() const;
	
	/** Allows media players to add additional analytics event attributes to an analytics event being recorded */
	UE_API virtual void GetAnalyticsEventAttributes(const FString& EventName, TArray<FAnalyticsEventAttribute>& InOutAttributes) const;
#endif

protected:
	/** Url used to open the media player. */
	FString OpenUrl;

	/** format of the video. */
	FMediaVideoTrackFormat VideoTrackFormat;

	/** format of the audio. */
	FMediaAudioTrackFormat AudioTrackFormat;

	/** Current state of the media player. */
	EMediaState CurrentState = EMediaState::Closed;

	/** Current playback time. */
	FTimespan CurrentTime = FTimespan::Zero();

	/** The media event handler. */
	IMediaEventSink& EventSink;

	/** Video frame rate in the last received sample. */
	FFrameRate VideoFrameRate = { 30, 1 };

	/** The media sample cache. */
	const TUniquePtr<FMediaIOCoreSamples> Samples;

	/** Sample evaluation type. */
	EMediaIOSampleEvaluationType EvaluationType = EMediaIOSampleEvaluationType::PlatformTime;

	/** Warn when the video frame rate is not the same as the engine's frame rate. */
	bool bWarnedIncompatibleFrameRate = false;

	/** Whether we are using autodetection. */
	bool bAutoDetect = false;

	/** When using Time Synchronization (TC synchronization), how many frame back of a delay would you like. */
	int32 FrameDelay = 0;

	/** When not using Time Synchronization (use computer time), how many sec back of a delay would you like. */
	double TimeDelay = 0.0;

	/** Previous frame Timespan */
	FTimespan PreviousFrameTimespan;

	/** Timed Data Input handler */
	TArray<ITimedDataInputChannel*> Channels;

	/** Base set of settings to start from when setuping channels */
	FMediaIOSamplingSettings BaseSettings;

	/** Open color IO conversion data. */
	TSharedPtr<struct FOpenColorIOColorConversionSettings> OCIOSettings;

	/** The media source that opened this player, for settings persistence. */
	TWeakObjectPtr<UObject> MediaSourceObject;

	/** Whether media playback should be framelocked to the engine's timecode */
	bool bFramelock = false;

	/** Used to ensure that JIT rendering is executed only once per frame */
	uint64 LastEngineRTFrameThatUpdatedJustInTime = TNumericLimits<uint64>::Max();
	
	/** Container for media source color settings. */
	TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> SourceColorSettings;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.8, "This property is deprecated. Please use SourceColorSettings instead.")
	bool bOverrideSourceEncoding = true;

	UE_DEPRECATED(5.8, "This property is deprecated. Please use SourceColorSettings instead.")
	ETextureSourceEncoding OverrideSourceEncoding = ETextureSourceEncoding::TSE_Linear;

	UE_DEPRECATED(5.8, "This property is deprecated. Please use SourceColorSettings instead.")
	bool bOverrideSourceColorSpace = true;

	UE_DEPRECATED(5.8, "This property is deprecated. Please use SourceColorSettings instead.")
	ETextureColorSpace OverrideSourceColorSpace = ETextureColorSpace::TCS_None;
#endif

private:
	UE_API void OnSampleDestroyed(TRefCountPtr<FRHITexture> InTexture);
	UE_API void RegisterSampleBuffer(const TSharedPtr<FMediaIOCoreTextureSampleBase>& InSample);
	UE_API void UnregisterSampleBuffers();
	UE_API void CreateAndRegisterTextures();
	UE_API void UnregisterTextures();
	UE_API void PreGPUTransfer(const TSharedPtr<FMediaIOCoreTextureSampleBase>& InSample);
	UE_API void PreGPUTransferJITR(const TSharedPtr<FMediaIOCoreTextureSampleBase>& InSample, const TSharedPtr<FMediaIOCoreTextureSampleBase>& InJITRProxySample);
	UE_API void ExecuteGPUTransfer(const TSharedPtr<FMediaIOCoreTextureSampleBase>& InSample);

	/** Get the right samples buffer for pushing samples depending on if we're in JITR mode or not. */
	UE_API FMediaIOCoreSamples& GetSamples_Internal();

	/** GPU Texture transfer object */
	UE::GPUTextureTransfer::TextureTransferPtr GPUTextureTransfer;

	/** Buffers Registered with GPU Texture Transfer */
	TSet<void*> RegisteredBuffers;

	/** Textures registerd with GPU Texture transfer. */
	TSet<TRefCountPtr<FRHITexture>> RegisteredTextures;

	/** Pool of textures available for GPU Texture transfer. */
	TArray<TRefCountPtr<FRHITexture>> Textures;

	/** Critical section to control access to the pool. */
	mutable FCriticalSection TexturesCriticalSection;

	/** Utility object to handle deinterlacing. */
	TSharedPtr<UE::MediaIOCore::FDeinterlacer> Deinterlacer;

private:

	/** Utility function to log various sample events (i.e. received/transferred/picked) */
	UE_API void LogBookmark(const FString& Text, const TSharedRef<IMediaTextureSample>& Sample);

private:

	/** Special JITR sample container, overrides FetchVideo to return a ProxySample that will be populated by JustInTimeSampleRender_RenderThread. */
	class FJITRMediaTextureSamples
		: public FMediaIOCoreSamples
	{
	public:
		FJITRMediaTextureSamples() = default;
		FJITRMediaTextureSamples(const FJITRMediaTextureSamples&) = delete;
		FJITRMediaTextureSamples& operator=(const FJITRMediaTextureSamples&) = delete;
	public:
		//~ Begin IMediaSamples interface
		virtual bool FetchVideo(TRange<FTimespan> TimeRange, TSharedPtr<IMediaTextureSample>& OutSample) override;
		virtual bool PeekVideoSampleTime(FMediaTimeStamp& TimeStamp) override;
		virtual void FlushSamples() override;
		//~ End IMediaSamples interface

	public:
		// JITR sample
		TSharedPtr<FMediaIOCoreTextureSampleBase> ProxySample;
	};

	/** Is Just-In-Time Rendering enabled */
	bool bJustInTimeRender = false;

	/** JITR samples proxy */
	const TUniquePtr<FJITRMediaTextureSamples> JITRSamples;
};

#undef UE_API
