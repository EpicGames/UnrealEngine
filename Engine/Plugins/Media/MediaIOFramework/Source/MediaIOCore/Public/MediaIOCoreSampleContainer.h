// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ITimedDataInput.h"

#include "CoreMinimal.h"
#include "IMediaSamples.h"
#include "IMediaTextureSample.h"
#include "ITimeManagementModule.h"
#include "MediaObjectPool.h"
#include "Misc/ScopeLock.h"
#include "Templates/SharedPointer.h"
#include "TimedDataInputCollection.h"


struct FMediaIOSamplingSettings
{
	FMediaIOSamplingSettings()
		: FrameRate(24, 1)
	{}

	FFrameRate FrameRate;
	ETimedDataInputEvaluationType EvaluationType = ETimedDataInputEvaluationType::PlatformTime;
	int32 BufferSize = 8;
	double PlayerTimeOffset = 0.0;
	int32 AbsoluteMaxBufferSize = 32;
};

/**
 * MediaIO container for different types of samples. Also a TimedData channel that can be monitored
 */
template<typename SampleType>
class FMediaIOCoreSampleContainer : public ITimedDataInputChannel
{
public:
	FMediaIOCoreSampleContainer(FName InChannelName)
		: ChannelName(InChannelName)
		, BufferUnderflow(0)
		, BufferOverflow(0)
		, FrameDrop(0)
		, bIsStatEnabled(true)
		, bIsChannelEnabled(false)
	{
	}

	FMediaIOCoreSampleContainer(const FMediaIOCoreSampleContainer&) = delete;
	FMediaIOCoreSampleContainer& operator=(const FMediaIOCoreSampleContainer&) = delete;
	virtual ~FMediaIOCoreSampleContainer() = default;

public:
	/** Update this sample container settings */
	void UpdateSettings(const FMediaIOSamplingSettings& InSettings)
	{
		EvaluationSettings = InSettings;
		ResetBufferStats();
	}

	/** Caches the current sample container states before samples will be taken out of it */
	void CacheState(FTimespan PlayerTime)
	{
		FScopeLock Lock(&CriticalSection);

		//Cache data for that tick. The player has decided the current time so we know the evaluation point. 
		const int32 SampleCount = Samples.Num();
		CachedSamplesData.Reset(SampleCount);
		if (SampleCount)
		{
			//Cache all times / timecode of each sample
			double CachedSampleDurationSeconds = 0.0;
			for (const TSharedPtr<SampleType, ESPMode::ThreadSafe>& Sample : Samples)
			{
				if (Sample.IsValid())
				{
					FTimedDataChannelSampleTime& NewSampleTime = CachedSamplesData.Emplace_GetRef();
					NewSampleTime.PlatformSecond = Sample->GetTime().Time.GetTotalSeconds() - EvaluationSettings.PlayerTimeOffset;
					if (Sample->GetTimecode().IsSet())
					{
						NewSampleTime.Timecode = FQualifiedFrameTime(Sample->GetTimecode().GetValue(), EvaluationSettings.FrameRate);
					}
				}
			}

			//Update statistics about evaluation time. Distance for newest sample will be clamped to 0 if its spans overlaps eval time.
			if (IsBufferStatsEnabled())
			{
				const double Duration = EvaluationSettings.FrameRate.AsInterval() - SMALL_NUMBER;

				double NewestSampleInSeconds = 0.0;
				double OldestSampleInSeconds = 0.0;
				const double EvaluationInSeconds = PlayerTime.GetTotalSeconds();
				if (EvaluationSettings.EvaluationType == ETimedDataInputEvaluationType::Timecode)
				{
					//Compute the distance with Timespan resolution. Going through FQualifiedFrameTime gives a different result (~10ns)
					if (Samples[0]->GetTimecode().IsSet())
					{
						NewestSampleInSeconds = Samples[0]->GetTimecode().GetValue().ToTimespan(EvaluationSettings.FrameRate).GetTotalSeconds() + Duration;
					}

					if (Samples[CachedSamplesData.Num() - 1]->GetTimecode().IsSet())
					{
						OldestSampleInSeconds = Samples[CachedSamplesData.Num() - 1]->GetTimecode().GetValue().ToTimespan(EvaluationSettings.FrameRate).GetTotalSeconds();
					}
				}
				else //Platform time
				{
					NewestSampleInSeconds = Samples[0]->GetTime().Time.GetTotalSeconds() + Duration;
					OldestSampleInSeconds = Samples[CachedSamplesData.Num() - 1]->GetTime().Time.GetTotalSeconds();
				}

				//Compute distance to evaluation taking into account duration of our samples for the newest one.
				CachedEvaluationData.DistanceToNewestSampleSeconds = NewestSampleInSeconds - EvaluationInSeconds;
				if (CachedEvaluationData.DistanceToNewestSampleSeconds >= 0.0 && CachedEvaluationData.DistanceToNewestSampleSeconds <= Duration)
				{
					CachedEvaluationData.DistanceToNewestSampleSeconds = 0.0;
				}

				//Oldest distance just uses delta with evaluation directly
				CachedEvaluationData.DistanceToOldestSampleSeconds = EvaluationInSeconds - OldestSampleInSeconds;

				if (!FMath::IsNearlyZero(CachedEvaluationData.DistanceToNewestSampleSeconds) && CachedEvaluationData.DistanceToNewestSampleSeconds < 0.0f)
				{
					++BufferOverflow;
				}

				if (!FMath::IsNearlyZero(CachedEvaluationData.DistanceToOldestSampleSeconds) && CachedEvaluationData.DistanceToOldestSampleSeconds < 0.0f)
				{
					++BufferUnderflow;
				}
			}
		}
	}

	/** Channel is disabled by default. It won't be added to the Timed Data collection if not enabled */
	void EnableChannel(ITimedDataInput* Input, bool bShouldEnable)
	{
		check(Input);

		if (bShouldEnable != bIsChannelEnabled)
		{
			bIsChannelEnabled = bShouldEnable;

			if (bIsChannelEnabled)
			{
				Input->AddChannel(this);
				ITimeManagementModule::Get().GetTimedDataInputCollection().Add(this);
			}
			else
			{
				ITimeManagementModule::Get().GetTimedDataInputCollection().Remove(this);
				Input->RemoveChannel(this);
			}
		}
	}

public:

	/**
	 * Add the given sample to the cache.
	 *
	 * @param Sample The sample to add.
	 * @return True if the operation succeeds.
	 */
	bool AddSample(const TSharedRef<SampleType, ESPMode::ThreadSafe>& Sample)
	{
		FScopeLock Lock(&CriticalSection);

		const int32 FutureSize = Samples.Num() + 1;
		if (FutureSize > EvaluationSettings.BufferSize)
		{
			++FrameDrop;
			Samples.RemoveAt(Samples.Num() - 1);
		}

		Samples.EmplaceAt(0, Sample);

		return true;
	}

	/**
	 * Pop a sample from the cache.
	 *
	 * @return True if the operation succeeds.
	 * @see AddSample, NumSamples
	 */
	bool PopSample()
	{
		FScopeLock Lock(&CriticalSection);
		const int32 SampleCount = Samples.Num();
		if (SampleCount > 0)
		{
			Samples.RemoveAt(SampleCount - 1);
			return true;
		}
		return false;
	}
	
	/**
	 * Get the number of queued samples.
	 *
	 * @return Number of samples.
	 * @see AddSample, PopSample
	 */
	int32 NumSamples() const
	{
		FScopeLock Lock(&CriticalSection);
		return Samples.Num();
	}

	/**
	 * Get next sample time from the sample list.
	 *
	 * @return Time of the next sample from the VideoSampleQueue
	 * @see AddSample, NumSamples
	 */
	FTimespan GetNextSampleTime()
	{
		FScopeLock Lock(&CriticalSection);

		const int32 SampleCount = Samples.Num();
		if (SampleCount > 0)
		{
			TSharedPtr<SampleType, ESPMode::ThreadSafe> Sample = Samples[SampleCount - 1];
			if (Sample.IsValid())
			{
				return Sample->GetTime().Time;
			}
		}

		return FTimespan();
	}

public:
	//~ Begin ITimedDataInputChannel
	
	virtual FText GetDisplayName() const override
	{
		return FText::FromName(ChannelName);
	}

	virtual ETimedDataInputState GetState() const override
	{
		return ETimedDataInputState::Connected;
	}

	virtual FTimedDataChannelSampleTime GetOldestDataTime() const override
	{
		if (CachedSamplesData.Num())
		{
			return CachedSamplesData[CachedSamplesData.Num() - 1];
		}

		return FTimedDataChannelSampleTime();
	}

	virtual FTimedDataChannelSampleTime GetNewestDataTime() const override
	{
		if (CachedSamplesData.Num())
		{
			//Newest data has to take into account the duration of the frame. A frame is considered valid when it's start time with duration overlaps the evaluation time
			const double Duration = EvaluationSettings.FrameRate.AsInterval() - SMALL_NUMBER;
			const FFrameTime SampleDurationTime = FFrameTime(FFrameNumber(0), 0.99f);

			FTimedDataChannelSampleTime ModifiedTime = CachedSamplesData[0];
			ModifiedTime.PlatformSecond += Duration;
			ModifiedTime.Timecode.Time += SampleDurationTime;
			return ModifiedTime;
		}

		return FTimedDataChannelSampleTime();
	}

	virtual TArray<FTimedDataChannelSampleTime> GetDataTimes() const override
	{
		return CachedSamplesData;
	}

	virtual int32 GetNumberOfSamples() const override
	{
		return CachedSamplesData.Num();
	}

	virtual int32 GetDataBufferSize() const override
	{
		return EvaluationSettings.BufferSize;
	}

	virtual void SetDataBufferSize(int32 BufferSize) override
	{
		FScopeLock Lock(&CriticalSection);
		EvaluationSettings.BufferSize = FMath::Clamp(BufferSize, 1, EvaluationSettings.AbsoluteMaxBufferSize);
		const int32 ToRemove = Samples.Num() - EvaluationSettings.BufferSize;
		for (int32 i = 0; i < ToRemove; ++i)
		{
			PopSample();
		}
	}

	virtual bool IsBufferStatsEnabled() const override
	{
		return bIsStatEnabled;
	}

	virtual void SetBufferStatsEnabled(bool bEnable) override
	{
		if (bEnable && !bIsStatEnabled)
		{
			//When enabling stat tracking, start clean
			ResetBufferStats();
		}

		bIsStatEnabled = bEnable;
	}

	virtual int32 GetBufferUnderflowStat() const override
	{
		return BufferUnderflow;
	}

	virtual int32 GetBufferOverflowStat() const override
	{
		return BufferOverflow;
	}

	virtual int32 GetFrameDroppedStat() const override
	{
		FScopeLock Lock(&CriticalSection);
		return FrameDrop;
	}

	virtual void GetLastEvaluationData(FTimedDataInputEvaluationData& OutEvaluationData) const override
	{
		OutEvaluationData = CachedEvaluationData;
	}

	virtual void ResetBufferStats() override
	{
		FScopeLock Lock(&CriticalSection);
		BufferUnderflow = 0;
		BufferOverflow = 0;
		FrameDrop = 0;
		CachedEvaluationData = FTimedDataInputEvaluationData();
	}

public:

	bool FetchSample(TRange<FTimespan> TimeRange, TSharedPtr<SampleType, ESPMode::ThreadSafe>& OutSample)
	{
		FScopeLock Lock(&CriticalSection);

		const int32 SampleCount = Samples.Num();
		if (SampleCount > 0)
		{
			TSharedPtr<SampleType, ESPMode::ThreadSafe> Sample = Samples[SampleCount - 1];
			if (Sample.IsValid())
			{
				const FTimespan SampleTime = Sample->GetTime().Time;

				if (TimeRange.Overlaps(TRange<FTimespan>(SampleTime, SampleTime + Sample->GetDuration())))
				{
					Samples.RemoveAt(SampleCount - 1);
					OutSample = Sample;
					return true;
				}
			}
		}

		return false;
	}

	/** Returns all available samples */
	TArray<TSharedPtr<SampleType, ESPMode::ThreadSafe>> GetSamples() const
	{
		FScopeLock Lock(&CriticalSection);
		return Samples;
	}

	void FlushSamples()
	{
		FScopeLock Lock(&CriticalSection);
		Samples.Empty();
	}


protected:

	/** Name of this channel */
	FName ChannelName;

	/** Samples are consumed by the the facade layer when evaluating and sending to render thread. Cache sample data before fetching happens */
	TArray<FTimedDataChannelSampleTime> CachedSamplesData;

	/** Last evaluation data for that channel */
	FTimedDataInputEvaluationData CachedEvaluationData;

	/** Evaluation statistics we keep track of */
	TAtomic<int32> BufferUnderflow;
	TAtomic<int32> BufferOverflow;
	TAtomic<int32> FrameDrop;

	/** Channel settings */
	FMediaIOSamplingSettings EvaluationSettings;

	/** Stats logging enabled or not */
	bool bIsStatEnabled;

	/** Should this channel be considered available */
	bool bIsChannelEnabled;

	/** 
	 * Sample container: We add at the beginning of the array [0] and we pop at the end [Size-1]. 
	 * [0] == Newest sample
	 * [Size - 1] == Oldest sample
	 */
	mutable FCriticalSection CriticalSection;
	TArray<TSharedPtr<SampleType, ESPMode::ThreadSafe>> Samples;
};
