// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IMediaSamples.h"
#include "IMediaTextureSample.h"
#include "MediaIOCoreSampleContainer.h"
#include "Misc/ScopeLock.h"
#include "Templates/SharedPointer.h"

#define UE_API MEDIAIOCORE_API

class IMediaAudioSample;
class IMediaBinarySample;
class IMediaOverlaySample;
class IMediaTextureSample;

/**
 * Bitflags about different types of sample a MediaIO can push
 */
enum class EMediaIOSampleType
{
	None = 0,
	Video = 1<<0,
	Audio = 1<<1,
	Metadata = 1<<2,
	Subtitles = 1<<3,
	Caption = 1<<4,
};
ENUM_CLASS_FLAGS(EMediaIOSampleType)

/**
 * General purpose media sample queue.
 */

class FMediaIOCoreSamples : public IMediaSamples
{
public:
	UE_API FMediaIOCoreSamples();
	FMediaIOCoreSamples(const FMediaIOCoreSamples&) = delete;
	FMediaIOCoreSamples& operator=(const FMediaIOCoreSamples&) = delete;

public:

	/**
	 * Add the given audio sample to the cache.
	 *
	 * @param Sample The sample to add.
	 * @return True if the operation succeeds.
	 * @see AddCaption, AddMetadata, AddSubtitle, AddVideo, PopAudio, NumAudioSamples
	 */
	bool AddAudio(const TSharedRef<IMediaAudioSample, ESPMode::ThreadSafe>& Sample)
	{
		return AudioSamples.AddSample(Sample);
	}

	/**
	 * Add the given caption sample to the cache.
	 *
	 * @param Sample The sample to add.
	 * @return True if the operation succeeds.
	 * @see AddAudio, AddMetadata, AddSubtitle, AddVideo, PopCaption, NumCaptionSamples
	 */
	bool AddCaption(const TSharedRef<IMediaOverlaySample, ESPMode::ThreadSafe>& Sample)
	{
		return CaptionSamples.AddSample(Sample);
	}

	/**
	 * Add the given metadata sample to the cache.
	 *
	 * @param Sample The sample to add.
	 * @return True if the operation succeeds.
	 * @see AddAudio, AddCaption, AddSubtitle, AddVideo, PopMetadata, NumMetadataSamples
	 */
	bool AddMetadata(const TSharedRef<IMediaBinarySample, ESPMode::ThreadSafe>& Sample)
	{
		return MetadataSamples.AddSample(Sample);
	}

	/**
	 * Add the given subtitle sample to the cache.
	 *
	 * @param Sample The sample to add.
	 * @return True if the operation succeeds.
	 * @see AddAudio, AddCaption, AddMetadata, AddVideo, PopSubtitle, NumSubtitleSamples
	 */
	bool AddSubtitle(const TSharedRef<IMediaOverlaySample, ESPMode::ThreadSafe>& Sample)
	{
		return SubtitleSamples.AddSample(Sample);
	}

	/**
	 * Add the given video sample to the cache.
	 *
	 * @param Sample The sample to add.
	 * @return True if the operation succeeds.
	 * @see AddAudio, AddCaption, AddMetadata, AddSubtitle, PopVideo, NumVideoSamples
	 */
	bool AddVideo(const TSharedRef<IMediaTextureSample, ESPMode::ThreadSafe>& Sample)
	{
		return VideoSamples.AddSample(Sample);
	}

	/**
	 * Pop a Audio sample from the cache.
	 *
	 * @return True if the operation succeeds.
	 * @see AddAudio, NumAudioSamples
	 */
	bool PopAudio()
	{
		return AudioSamples.PopSample();
	}

	/**
	 * Pop a Caption sample from the cache.
	 *
	 * @return True if the operation succeeds.
	 * @see AddCaption, NumCaption
	 */
	bool PopCaption()
	{
		return CaptionSamples.PopSample();
	}

	/**
	 * Pop a Metadata sample from the cache.
	 *
	 * @return True if the operation succeeds.
	 * @see AddMetadata, NumMetadata
	 */
	bool PopMetadata()
	{
		return MetadataSamples.PopSample();
	}

	/**
	 * Pop a Subtitle sample from the cache.
	 *
	 * @return True if the operation succeeds.
	 * @see AddSubtitle, NumSubtitle
	 */
	bool PopSubtitle()
	{
		return SubtitleSamples.PopSample();
	}

	/**
	 * Pop a video sample from the cache.
	 *
	 * @return True if the operation succeeds.
	 * @see AddVideo, NumVideo
	 */
	bool PopVideo()
	{
		return VideoSamples.PopSample();
	}

	/**
	 * Get the number of queued audio samples.
	 *
	 * @return Number of samples.
	 * @see AddAudio, PopAudio
	 */
	int32 NumAudioSamples() const
	{
		return AudioSamples.NumSamples();
	}

	/**
	 * Get the number of queued caption samples.
	 *
	 * @return Number of samples.
	 * @see AddCaption, PopCaption
	 */
	int32 NumCaptionSamples() const
	{
		return CaptionSamples.NumSamples();
	}

	/**
	 * Get the number of queued metadata samples.
	 *
	 * @return Number of samples.
	 * @see AddMetadata, PopMetada
	 */
	int32 NumMetadataSamples() const
	{
		return MetadataSamples.NumSamples();
	}

	/**
	 * Get the number of queued subtitle samples.
	 *
	 * @return Number of samples.
	 * @see AddSubtitle, PopSubtitle
	 */
	int32 NumSubtitleSamples() const
	{
		return SubtitleSamples.NumSamples();
	}

	/**
	 * Get the number of queued video samples.
	 *
	 * @return Number of samples.
	 * @see AddVideo, PopVideo
	 */
	int32 NumVideoSamples() const
	{
		return VideoSamples.NumSamples();
	}

	/**
	 * Get next sample time from the VideoSampleQueue.
	 *
	 * @return Time of the next sample from the VideoSampleQueue
	 * @see AddVideo, NumVideoSamples
	 */
	FTimespan GetNextVideoSampleTime()
	{
		return VideoSamples.GetNextSampleTime();
	}

	/**
	 * Get Audio Samples frame dropped count.
	 *
	 * @return Number of frames dropped
	 */
	int32 GetAudioFrameDropCount() const
	{
		return AudioSamples.GetFrameDroppedStat();
	}

	/**
	 * Get Video Samples frame dropped count.
	 *
	 * @return Number of frames dropped
	 */
	int32 GetVideoFrameDropCount() const
	{
		return VideoSamples.GetFrameDroppedStat();
	}
	
	/**
	 * Get Metadata Samples frame dropped count.
	 *
	 * @return Number of frames dropped
	 */
	int32 GetMetadataFrameDropCount() const
	{
		return MetadataSamples.GetFrameDroppedStat();
	}

	/**
	 * Get Subtitles Samples frame dropped count.
	 *
	 * @return Number of frames dropped
	 */
	int32 GetSubtitlesFrameDropCount() const
	{
		return SubtitleSamples.GetFrameDroppedStat();
	}

	/**
	 * Get Caption Samples frame dropped count.
	 *
	 * @return Number of frames dropped
	 */
	int32 GetCaptionsFrameDropCount() const
	{
		return CaptionSamples.GetFrameDroppedStat();
	}

	/**
	 * Returns all available video samples
	 */
	TArray<TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>> GetVideoSamples() const
	{
		return VideoSamples.GetSamples();
	}

	/**
	 * Caches the current sample container state for a given Player (evaluation) time
	 */
	UE_API void CacheSamplesState(FTimespan PlayerTime);

	/** Enable or disable channels based on bitfield flag */
	UE_API void EnableTimedDataChannels(ITimedDataInput* Input, EMediaIOSampleType SampleTypes);

	/** Initialize our different buffers with player's settings and wheter or not it should be displayed / supported in Timing monitor */
	UE_API void InitializeVideoBuffer(const FMediaIOSamplingSettings& InSettings);
	UE_API void InitializeAudioBuffer(const FMediaIOSamplingSettings& InSettings);
	UE_API void InitializeMetadataBuffer(const FMediaIOSamplingSettings& InSettings);
	UE_API void InitializeSubtitlesBuffer(const FMediaIOSamplingSettings& InSettings);
	UE_API void InitializeCaptionBuffer(const FMediaIOSamplingSettings& InSettings);

public:

	//~ IMediaSamples interface

	UE_API virtual bool FetchAudio(TRange<FTimespan> TimeRange, TSharedPtr<IMediaAudioSample, ESPMode::ThreadSafe>& OutSample) override;
	UE_API virtual bool FetchCaption(TRange<FTimespan> TimeRange, TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>& OutSample) override;
	UE_API virtual bool FetchMetadata(TRange<FTimespan> TimeRange, TSharedPtr<IMediaBinarySample, ESPMode::ThreadSafe>& OutSample) override;
	UE_API virtual bool FetchSubtitle(TRange<FTimespan> TimeRange, TSharedPtr<IMediaOverlaySample, ESPMode::ThreadSafe>& OutSample) override;
	UE_API virtual bool FetchVideo(TRange<FTimespan> TimeRange, TSharedPtr<IMediaTextureSample, ESPMode::ThreadSafe>& OutSample) override;
	UE_API virtual void FlushSamples() override;
	UE_API virtual bool PeekVideoSampleTime(FMediaTimeStamp & TimeStamp) override;
	UE_API virtual void SetSampleBufferSize(int32 BufferSize) override;

protected:

	FMediaIOCoreSampleContainer<IMediaTextureSample> VideoSamples;
	FMediaIOCoreSampleContainer<IMediaAudioSample> AudioSamples;
	FMediaIOCoreSampleContainer<IMediaBinarySample> MetadataSamples;
	FMediaIOCoreSampleContainer<IMediaOverlaySample> SubtitleSamples;
	FMediaIOCoreSampleContainer<IMediaOverlaySample> CaptionSamples;
};

#undef UE_API
