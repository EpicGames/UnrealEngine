// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioDeviceHandle.h"

#include "AudioDeviceHandle.h"
#include "MediaIOCoreAudioOutput.h"
#include "Subsystems/EngineSubsystem.h"

#include "MediaIOCoreSubsystem.generated.h"

#define UE_API MEDIAIOCORE_API

UCLASS(MinimalAPI)
class UMediaIOCoreSubsystem : public UEngineSubsystem
{
public:
	DECLARE_TS_MULTICAST_DELEGATE_ThreeParams(FOnBufferReceived, Audio::FDeviceId /* DeviceId */, float* /* Data */, int32 /* NumSamples */)
	
	struct FCreateAudioOutputArgs
	{
		uint32 NumOutputChannels = 0;
		FFrameRate TargetFrameRate; 
		uint32 MaxSampleLatency = 0;
		uint32 OutputSampleRate = 0;
		FAudioDeviceHandle AudioDeviceHandle;
	};

public:
	GENERATED_BODY()

	//~ Begin UEngineSubsystem Interface
	UE_API virtual void Initialize(FSubsystemCollectionBase& InCollection) override;
	UE_API virtual void Deinitialize() override;
	//~ End UEngineSubsystem Interface

	/**
	 * Create an audio output that allows getting audio that was accumulated during the last frame. 
	 */
	UE_API TSharedPtr<FMediaIOAudioOutput> CreateAudioOutput(const FCreateAudioOutputArgs& InArgs);

	/**
	 * Get the number of audio channels used by the main audio device.
	 **/
	UE_API int32 GetNumAudioInputChannels() const;

	/**
	 * @Note: Called from the audio thread.
	 */
	FOnBufferReceived& OnBufferReceived_AudioThread()
	{
		return BufferReceivedDelegate;
	}

private:
	UE_API void OnAudioDeviceDestroyed(Audio::FDeviceId InAudioDeviceId);
	UE_API void OnBufferReceivedByCapture(float* Data, int32 NumSamples, Audio::FDeviceId AudioDeviceID) const;

private:
	TSharedPtr<FMainMediaIOAudioCapture> MainMediaIOAudioCapture;
	
	TMap<Audio::FDeviceId, TSharedPtr<FMediaIOAudioCapture, ESPMode::ThreadSafe>> MediaIOAudioCaptures;

	FDelegateHandle DeviceDestroyedHandle;

	/** Delegate called from the audio thread when a buffer is received. Must be thread safe. */
	FOnBufferReceived BufferReceivedDelegate;
};

#undef UE_API
