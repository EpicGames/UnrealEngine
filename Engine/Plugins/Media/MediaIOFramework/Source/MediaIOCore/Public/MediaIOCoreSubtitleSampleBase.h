// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaOverlaySample.h"
#include "MediaObjectPool.h"


/**
 * Implements a media subtitle sample.
 */
class FMediaIOCoreSubtitleSampleBase
	: public IMediaOverlaySample
	, public IMediaPoolable
{
};
