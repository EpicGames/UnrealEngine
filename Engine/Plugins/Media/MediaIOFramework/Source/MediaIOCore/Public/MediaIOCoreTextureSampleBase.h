// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaTextureSample.h"
#include "BaseMediaSourceColorSettings.h"
#include "MediaIOCoreDefinitions.h"
#include "IMediaTextureSampleConverter.h"
#include "MediaObjectPool.h"
#include "Misc/FrameRate.h"
#include "Templates/RefCounting.h"

#define UE_API MEDIAIOCORE_API

class FMediaIOCorePlayerBase;
class FMediaIOCoreTextureSampleConverter;
class FRHITexture;

namespace UE::MediaIOCore
{
	struct UE_DEPRECATED(5.8, "FColorFormatArgs is deprecated. Please use FNativeMediaSourceColorSettings instead.") FColorFormatArgs
	{
		FColorFormatArgs() = default;
		
		/** Bool constructor to allow backwards compatibility with previous method definitions of FMediaIOCoreTextureSampleBase::Initialize. */
		FColorFormatArgs(bool bIsSRGBInput)
		{
			if (bIsSRGBInput)
			{
				Encoding = UE::Color::EEncoding::sRGB;
				ColorSpaceType = UE::Color::EColorSpace::sRGB;
			}
			else
			{
				Encoding = UE::Color::EEncoding::Linear;
				ColorSpaceType = UE::Color::EColorSpace::sRGB;
			}
		}

		FColorFormatArgs(UE::Color::EEncoding InEncoding, UE::Color::EColorSpace InColorSpaceType)
			: Encoding(InEncoding)
			, ColorSpaceType(InColorSpaceType)
		{
		}
		

		/** Encoding of the texture. */
		UE::Color::EEncoding Encoding = UE::Color::EEncoding::Linear;

		/** Color space of the texture. */
		UE::Color::EColorSpace ColorSpaceType = UE::Color::EColorSpace::sRGB;
	};
}



/** Used to setup JITR data for a sample */
struct FMediaIOCoreSampleJITRConfigurationArgs
{
	/** Width of the sample in pixels */
	uint32 Width = 0;

	/** Height of the sample in pixels */
	uint32 Height = 0;

	/** Platform time fixed on the game thread for JITR sample picking */
	FTimespan Time;

	/** Engine timecode fixed on the game thread for JITR sample picking */
	FTimecode Timecode;

	/** Time offset evaluated on game thread */
	double EvaluationOffsetInSeconds = 0;

	/** Player that produced the sample */
	TSharedPtr<FMediaIOCorePlayerBase> Player;

	/** Sample converter to process this sample */
	TSharedPtr<FMediaIOCoreTextureSampleConverter> Converter;
	
	/** Frame rate of the current sample. */
	FFrameRate FrameRate;
};


/**
 * Implements the IMediaTextureSample/IMediaPoolable interface.
 */
class FMediaIOCoreTextureSampleBase
	: public IMediaTextureSample
	, public IMediaPoolable
	, public IMediaTextureSampleColorConverter
	, public TSharedFromThis<FMediaIOCoreTextureSampleBase, ESPMode::ThreadSafe>
{

public:
	UE_API FMediaIOCoreTextureSampleBase();

	/**
	 * Initialize the sample.
	 *
	 * @param InVideoBuffer The video frame data.
	 * @param InBufferSize The size of the video buffer.
	 * @param InStride The number of channel of the video buffer.
	 * @param InWidth The sample rate of the video buffer.
	 * @param InHeight The sample rate of the video buffer.
	 * @param InSampleFormat The sample format of the video buffer.
	 * @param InTime The sample time (in the player's own clock).
	 * @param InFrameRate The framerate of the media that produce the sample.
	 * @param InTimecode The sample timecode if available.
	 * @param InSourceColorSettings Information about the texture color encoding and color space.
	 */
	UE_API bool Initialize(const void* InVideoBuffer, uint32 InBufferSize, uint32 InStride, uint32 InWidth, uint32 InHeight, EMediaTextureSampleFormat InSampleFormat, FTimespan InTime, const FFrameRate& InFrameRate, const TOptional<FTimecode>& InTimecode, TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> InSourceColorSettings);

	/**
	 * Initialize the sample.
	 *
	 * @param InVideoBuffer The video frame data.
	 * @param InStride The number of channel of the video buffer.
	 * @param InWidth The sample rate of the video buffer.
	 * @param InHeight The sample rate of the video buffer.
	 * @param InSampleFormat The sample format of the video buffer.
	 * @param InTime The sample time (in the player's own clock).
	 * @param InFrameRate The framerate of the media that produce the sample.
	 * @param InTimecode The sample timecode if available.
	 * @param InSourceColorSettings Information about the texture color encoding and color space.
	 */
	UE_API bool Initialize(const TArray<uint8>& InVideoBuffer, uint32 InStride, uint32 InWidth, uint32 InHeight, EMediaTextureSampleFormat InSampleFormat, FTimespan InTime, const FFrameRate& InFrameRate, const TOptional<FTimecode>& InTimecode, TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> InSourceColorSettings);

	/**
	 * Initialize the sample.
	 *
	 * @param InVideoBuffer The video frame data.
	 * @param InStride The number of channel of the video buffer.
	 * @param InWidth The sample rate of the video buffer.
	 * @param InHeight The sample rate of the video buffer.
	 * @param InSampleFormat The sample format of the video buffer.
	 * @param InTime The sample time (in the player's own clock).
	 * @param InFrameRate The framerate of the media that produce the sample.
	 * @param InTimecode The sample timecode if available.
	 * @param InSourceColorSettings Information about the texture color encoding and color space.
	 */
	UE_API bool Initialize(TArray<uint8>&& InVideoBuffer, uint32 InStride, uint32 InWidth, uint32 InHeight, EMediaTextureSampleFormat InSampleFormat, FTimespan InTime, const FFrameRate& InFrameRate, const TOptional<FTimecode>& InTimecode, TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> InSourceColorSettings);

	/**
	 * Initialize the sample.
	 *
	 * @param InVideoBuffer The video frame data.
	 * @param InBufferSize The size of the video buffer.
	 */
	UE_API bool SetBuffer(const void* InVideoBuffer, uint32 InBufferSize);

	/**
	 * Set the sample buffer.
	 *
	 * @param InVideoBuffer The video frame data.
	 */
	UE_API bool SetBuffer(const TArray<uint8>& InVideoBuffer);

	/**
	 * Set the sample buffer.
	 *
	 * @param InVideoBuffer The video frame data.
	 */
	UE_API bool SetBuffer(TArray<uint8>&& InVideoBuffer);

	/**
	 * Set the sample properties.
	 *
	 * @param InStride The number of channel of the video buffer.
	 * @param InWidth The width the video buffer.
	 * @param InHeight The height of the video buffer.
	 * @param InSampleFormat The sample format of the video buffer.
	 * @param InTime The sample time (in the player's own clock).
	 * @param InFrameRate The framerate of the media that produce the sample.
	 * @param InTimecode The sample timecode if available.
	 * @param InSourceColorSettings Information about the texture color encoding and color space.
	 */
	UE_API bool SetProperties(uint32 InStride, uint32 InWidth, uint32 InHeight, EMediaTextureSampleFormat InSampleFormat, FTimespan InTime, const FFrameRate& InFrameRate, const TOptional<FTimecode>& InTimecode, TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> InSourceColorSettings);

	/**
	 * Initialize the sample with half it's original height and take only the odd or even line.
	 *
	 * @param bUseEvenLine Should use the Even or the Odd line from the video buffer.
	 * @param InVideoBuffer The video frame data.
	 * @param InStride The number of channel of the video buffer.
	 * @param InWidth The width of the video buffer.
	 * @param InHeight The height of the video buffer.
	 * @param InSampleFormat The sample format of the video buffer.
	 * @param InTime The sample time (in the player's own clock).
	 * @param InTimecode The sample timecode if available.
	 * @param InSourceColorSettings Information about the texture color encoding and color space.
	 */
	UE_API bool InitializeWithEvenOddLine(bool bUseEvenLine, const void* InVideoBuffer, uint32 InBufferSize, uint32 InStride, uint32 InWidth, uint32 InHeight, EMediaTextureSampleFormat InSampleFormat, FTimespan InTime, const FFrameRate& InFrameRate, const TOptional<FTimecode>& InTimecode, TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> InSourceColorSettings);
	
	/**
	 * Set the sample buffer with half it's original height and take only the odd or even line.
	 *
	 * @param bUseEvenLine Should use the Even or the Odd line from the video buffer.
	 * @param InVideoBuffer The video frame data.
	 * @param InBufferSize The size of the video buffer.
	 * @param InStride The number of channel of the video buffer.
	 * @param InHeight The height of the video buffer.
	 */
	UE_API bool SetBufferWithEvenOddLine(bool bUseEvenLine, const void* InVideoBuffer, uint32 InBufferSize, uint32 InStride, uint32 InHeight);

	/**
	 * Set the OCIO settings used for color conversion.
	 */
	UE_API void SetColorConversionSettings(TSharedPtr<struct FOpenColorIOColorConversionSettings> InColorConversionSettings);

	/**
	 * Request an uninitialized sample buffer.
	 * Should be used when the buffer could be filled by something else.
	 * SetProperties should still be called after.
	 *
	 * @param InBufferSize The size of the video buffer.
	 */
	UE_API virtual void* RequestBuffer(uint32 InBufferSize);


	/**
	 * Configure this sample for JITR
	 *
	 * @param Args JITR configuration parameters.
	 */
	UE_API virtual bool InitializeJITR(const FMediaIOCoreSampleJITRConfigurationArgs& Args);

	/** Marks this sample as one that is ready and awaiting for fast GPUDirect texture transfer */
	void SetAwaitingForGPUTransfer(bool bIsAwaitingGPUTransfer = true)
	{
		bIsAwaitingForGPUTransfer = bIsAwaitingGPUTransfer;
	}

	/** Returns whether it's ready for GPUDirect texture transfer */
	bool IsAwaitingForGPUTransfer() const
	{
		return bIsAwaitingForGPUTransfer;
	}

	/** Returns the player that created this sample */
	UE_API TSharedPtr<FMediaIOCorePlayerBase> GetPlayer() const;

	/** Returns time evaluated on the game thread for JITR */
	double GetEvaluationOffsetInSeconds() const
	{
		return EvaluationOffsetInSeconds;
	}

	/** Copies all neccessary data from a source sample to render JIT */
	UE_API virtual void CopyConfiguration(const TSharedPtr<FMediaIOCoreTextureSampleBase>& SourceSample);

	//~ Deprecated methods
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "Deprecated method. Please use method with FNativeMediaSourceColorSettings argument instead.")
	UE_API bool Initialize(const void* InVideoBuffer, uint32 InBufferSize, uint32 InStride, uint32 InWidth, uint32 InHeight, EMediaTextureSampleFormat InSampleFormat, FTimespan InTime, const FFrameRate& InFrameRate, const TOptional<FTimecode>& InTimecode, const UE::MediaIOCore::FColorFormatArgs& InColorFormatArgs);
	UE_DEPRECATED(5.8, "Deprecated method. Please use method with FNativeMediaSourceColorSettings argument instead.")
	UE_API bool Initialize(const TArray<uint8>& InVideoBuffer, uint32 InStride, uint32 InWidth, uint32 InHeight, EMediaTextureSampleFormat InSampleFormat, FTimespan InTime, const FFrameRate& InFrameRate, const TOptional<FTimecode>& InTimecode, const UE::MediaIOCore::FColorFormatArgs& InColorFormatArgs);
	UE_DEPRECATED(5.8, "Deprecated method. Please use method with FNativeMediaSourceColorSettings argument instead.")
	UE_API bool Initialize(TArray<uint8>&& InVideoBuffer, uint32 InStride, uint32 InWidth, uint32 InHeight, EMediaTextureSampleFormat InSampleFormat, FTimespan InTime, const FFrameRate& InFrameRate, const TOptional<FTimecode>& InTimecode, const UE::MediaIOCore::FColorFormatArgs& InColorFormatArgs);
	UE_DEPRECATED(5.8, "Deprecated method. Please use method with FNativeMediaSourceColorSettings argument instead.")
	UE_API bool SetProperties(uint32 InStride, uint32 InWidth, uint32 InHeight, EMediaTextureSampleFormat InSampleFormat, FTimespan InTime, const FFrameRate& InFrameRate, const TOptional<FTimecode>& InTimecode, const UE::MediaIOCore::FColorFormatArgs& InColorFormatArgs);
	UE_DEPRECATED(5.8, "Deprecated method. Please use method with FNativeMediaSourceColorSettings argument instead.")
	UE_API bool InitializeWithEvenOddLine(bool bUseEvenLine, const void* InVideoBuffer, uint32 InBufferSize, uint32 InStride, uint32 InWidth, uint32 InHeight, EMediaTextureSampleFormat InSampleFormat, FTimespan InTime, const FFrameRate& InFrameRate, const TOptional<FTimecode>& InTimecode, const UE::MediaIOCore::FColorFormatArgs& InColorFormatArgs);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

protected:
	/**
	* Method that caches color conversion settings on Game thread.
	*/
	UE_API void CacheColorCoversionSettings_GameThread();

public:
	//~ IMediaTextureSample interface

	virtual FIntPoint GetDim() const override
	{
		switch (GetFormat())
		{
		case EMediaTextureSampleFormat::CharAYUV:
		case EMediaTextureSampleFormat::CharNV12:
		case EMediaTextureSampleFormat::CharNV21:
		case EMediaTextureSampleFormat::CharUYVY:
		case EMediaTextureSampleFormat::CharYUY2:
		case EMediaTextureSampleFormat::CharYVYU:
			return FIntPoint(Width / 2, Height);
		case EMediaTextureSampleFormat::YUVv210:
			// Data for 6 output pixels is contained in 4 actual texture pixels
			// Padding aligned on 48 (16 and 6 at the same time)
			return FIntPoint(4 * ((((Width + 47) / 48) * 48) / 6), Height);
		default:
			return FIntPoint(Width, Height);
		}
	}

	virtual FTimespan GetDuration() const override
	{
		return Duration;
	}

	virtual EMediaTextureSampleFormat GetFormat() const override
	{
		return SampleFormat;
	}

	virtual FIntPoint GetOutputDim() const override
	{
		return FIntPoint(Width, Height);
	}

	virtual uint32 GetStride() const override
	{
		return Stride;
	}

	virtual FMediaTimeStamp GetTime() const override
	{
		return FMediaTimeStamp(Time);
	}

	virtual TOptional<FTimecode> GetTimecode() const override
	{
		return Timecode;
	}

	virtual bool IsCacheable() const override
	{
		return true;
	}

	UE_API virtual bool IsOutputSrgb() const override;

	UE_API virtual bool HasEncodingOverride() const;
	UE_API virtual bool HasColorSpaceOverride() const;
	UE_API virtual const UE::Color::FColorSpace& GetSourceColorSpace() const override;
	UE_API virtual UE::Color::EEncoding GetEncodingType() const override;
	UE_API virtual UE::Color::EChromaticAdaptationMethod GetChromaticAdapationMethod() const override;
	UE_API virtual float GetHDRNitsNormalizationFactor() const override;
	
	UE_DEPRECATED(5.8, "GetColorSpaceType is deprecated, please use GetSourceColorSpace instead.")
	UE_API virtual UE::Color::EColorSpace GetColorSpaceType() const;

	virtual const void* GetBuffer() override
	{
		// Don't return the buffer if we have a texture to force the media player to use the texture if available. 
		if (Texture)
		{
			return nullptr;
		}

		if (ExternalBuffer)
		{
			return ExternalBuffer;
		}

		return Buffer.GetData();
	}

	/**
	 * Attemps to get the initialized buffer.
	 * If buffer isn't initialized or of a different size, requests a new one.
	 *
	 * @param InBufferSize The size of the required video buffer.
	 */
	UE_API virtual void* GetOrRequestBuffer(uint32 InBufferSize);


	virtual uint64 GetFrameNumber() const
	{
		return FrameNumber;
	}

	virtual void SetTime(const FTimespan& InTime)
	{
		Time = InTime;
	}

	virtual void SetFrameNumber(uint32 InFrameNumber)
	{
		FrameNumber = InFrameNumber;
	}

	//~ IMediaTextureSampleColorConverter interface
	UE_API virtual bool ApplyColorConversion(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InSrcTexture, FTextureRHIRef& InDstTexture) override;

	void* GetMutableBuffer()
	{
		if (ExternalBuffer)
		{
			return ExternalBuffer;
		}

		return Buffer.GetData();
	}

#if WITH_ENGINE
	UE_API virtual IMediaTextureSampleConverter* GetMediaTextureSampleConverter() override;
	UE_API virtual FRHITexture* GetTexture() const override;

	UE_API virtual IMediaTextureSampleColorConverter* GetMediaTextureSampleColorConverter() override;
#endif //WITH_ENGINE

	void SetBuffer(void* InBuffer)
	{
		ExternalBuffer = InBuffer;
	}

	UE_API void SetTexture(TRefCountPtr<FRHITexture> InRHITexture);
	UE_API void SetDestructionCallback(TFunction<void(TRefCountPtr<FRHITexture>)> InDestructionCallback);
	UE_API EPixelFormat GetPixelFormat();
private:
	/** Hold a texture to be used for gpu texture transfers. */
	TRefCountPtr<FRHITexture> Texture;

	/** Called when the sample is destroyed by its pool. */
	TFunction<void(TRefCountPtr<FRHITexture>)> DestructionCallback;

public:
	//~ IMediaPoolable interface

	UE_API virtual void ShutdownPoolable() override;

protected:
	virtual void FreeSample()
	{
		Buffer.Reset();
		ExternalBuffer = nullptr;
		CachedOCIOResources.Reset();
		Texture.SafeRelease();
		OriginalSample.Reset();
	}

	/**
	 * Get YUV to RGB conversion matrix
	 *
	 * @return MediaIOCore Yuv To Rgb matrix
	 */
	UE_API virtual const FMatrix& GetYUVToRGBMatrix() const override;

protected:
	/** Duration for which the sample is valid. */
	FTimespan Duration;

	/** Sample format. */
	EMediaTextureSampleFormat SampleFormat;

	/** Sample time. */
	FTimespan Time;

	/** Sample timecode. */
	TOptional<FTimecode> Timecode;

	/** Which engine frame number this sample corresponds to. */
	std::atomic<uint64> FrameNumber;

	/** Image dimensions */
	uint32 Stride = 0;
	uint32 Width  = 0;
	uint32 Height = 0;

	/** Pointer to raw pixels */
	TArray<uint8, TAlignedHeapAllocator<4096>> Buffer;
	void* ExternalBuffer = nullptr;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.8, "Please use SourceColorSettings instead.")
	UE::Color::EEncoding Encoding = UE::Color::EEncoding::Linear;

	UE_DEPRECATED(5.8, "Please use SourceColorSettings instead.")
	UE::Color::EColorSpace ColorSpaceType = UE::Color::EColorSpace::sRGB;

	UE_DEPRECATED(5.8, "Please use SourceColorSettings instead.")
	UE::Color::FColorSpace ColorSpaceStruct = UE::Color::FColorSpace(UE::Color::EColorSpace::sRGB);
#endif

	/** The player that created this sample */
	TWeakPtr<FMediaIOCorePlayerBase> Player;

	/** Custom converter that will be the one checking back with the player for just in time sample render purposes */
	TSharedPtr<FMediaIOCoreTextureSampleConverter> Converter;

private:
	/**
	 * A reference to the original sample that was chosen during JITR. The idea of this member is to keep
	 * the original sample alive, prevent any of its resources from being released while this proxy sample is in use.
	 */
	TSharedPtr<FMediaIOCoreTextureSampleBase> OriginalSample;

	/** Whether this sample's texture data is awaiting to be transferred by GPUDirect */
	std::atomic<bool> bIsAwaitingForGPUTransfer;

	/** Time offset evaluated on game thread for JITR */
	double EvaluationOffsetInSeconds = 0;

private:
	/** Manual source color space & encoding overrides. */
	TSharedPtr<FNativeMediaSourceColorSettings, ESPMode::ThreadSafe> SourceColorSettings;
	/** Settings used to apply an OCIO conversion to the sample. */
	TSharedPtr<struct FOpenColorIOColorConversionSettings> ColorConversionSettings;
	/** Cached rendering resources used for the color conversion pass when using OCIO. */
	TSharedPtr<struct FOpenColorIORenderPassResources> CachedOCIOResources;
};

#undef UE_API
