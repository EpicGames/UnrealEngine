// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMediaTextureSampleConverter.h"

#define UE_API MEDIAIOCORE_API

class FMediaIOCoreTextureSampleBase;


/**
 * Media IO base texture sample converter.
 *
 * It's mostly responsible for Just-In-Time Rendering (JITR), but also provides a way
 * to implement custom sample conversion when inherited.
 * 
 * @note: Don't forget to call FMediaIOTextureSampleConverter::Convert(...) from the child classes
 *        to trigger JITR pipeline. This is the place where late sample picking is actually happens.
 */
class FMediaIOCoreTextureSampleConverter
	: public IMediaTextureSampleConverter
{
public:
	FMediaIOCoreTextureSampleConverter() = default;
	virtual ~FMediaIOCoreTextureSampleConverter() = default;

public:
	/** Configures settings to convert incoming sample */
	UE_API virtual void Setup(const TSharedPtr<FMediaIOCoreTextureSampleBase>& InSample);

public:
	//~ Begin IMediaTextureSampleConverter interface
	UE_API virtual bool Convert(FRHICommandListImmediate& RHICmdList, FTextureRHIRef& InDstTexture, const FConversionHints& Hints) override;
	UE_API virtual uint32 GetConverterInfoFlags() const override;
	//~ End IMediaTextureSampleConverter interface

protected:

	/** Proxy sample for JITR */
	TWeakPtr<FMediaIOCoreTextureSampleBase> JITRProxySample;
};

#undef UE_API
