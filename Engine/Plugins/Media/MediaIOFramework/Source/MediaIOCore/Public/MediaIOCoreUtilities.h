// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreGlobals.h"
#include "Containers/UnrealString.h"
#include "Containers/Map.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Logging/LogVerbosity.h"
#include "HAL/PlatformTime.h"
#include "HAL/CriticalSection.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/ScopeLock.h"
#include "Misc/Timespan.h"

namespace UE::MediaIO
{
#if NO_LOGGING
	static void LogThrottle(const FNoLoggingCategory& InLogCategory, ELogVerbosity::Type InVerbosity, const FTimespan& InTimeBetweenLogs, const FString& LogDetails, const FString& FileName, int32 LineNumber);
#else
	static void LogThrottle(const FLogCategoryBase& InLogCategory, ELogVerbosity::Type InVerbosity, const FTimespan& InTimeBetweenLogs, const FString& LogDetails, const FString& FileName, int32 LineNumber);
#endif
}

#define UE_MEDIA_IO_LOG_THROTTLE(LogCategory, Verbosity, TimeBetweenLogs, Format, ...) (UE::MediaIO::LogThrottle(LogCategory, ELogVerbosity::Verbosity, TimeBetweenLogs, *FString::Printf(Format, ##__VA_ARGS__), FString(__FILE__), __LINE__))

namespace UE::MediaIO
{
#if NO_LOGGING
	static void LogThrottle(const FNoLoggingCategory& InLogCategory, ELogVerbosity::Type InVerbosity, const FTimespan& InTimeBetweenLogs, const FString& LogDetails, const FString& FileName, int32 LineNumber)
	{
	}
#else
	static void LogThrottle(const FLogCategoryBase& InLogCategory, ELogVerbosity::Type InVerbosity, const FTimespan& InTimeBetweenLogs, const FString& LogDetails, const FString& FileName, int32 LineNumber)
	{
		// This code was adapted from AudioMixer.h to support throttling instead of only logging once.
		// Log once to avoid Spam.
		static FCriticalSection Cs;
		static TMap<uint32, double> LogHistory;

		FScopeLock Lock(&Cs);
		const FString MessageToHash = FString::Printf(TEXT("%s (File %s, Line %d)"), *LogDetails, *FileName, LineNumber);

		uint32 Hash = GetTypeHash(MessageToHash);

		bool bShouldLog = false;
		double* LastLogTime = LogHistory.Find(Hash);
		
		double NewTime = FPlatformTime::Seconds();
		if (LastLogTime)
		{
			const double TimeDiff = NewTime - *LastLogTime;
			if (TimeDiff > InTimeBetweenLogs.GetTotalSeconds())
			{
				*LastLogTime = NewTime;
				bShouldLog = true;
			}
		}
		else
		{
			bShouldLog = true;
			LogHistory.Add(Hash, NewTime);
		}

		if (bShouldLog)
		{
			GLog->Log(InLogCategory.GetCategoryName(), InVerbosity, LogDetails);
		}
	}
#endif
}
