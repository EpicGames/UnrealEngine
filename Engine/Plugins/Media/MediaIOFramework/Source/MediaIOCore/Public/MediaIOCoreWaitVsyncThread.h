// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineBaseTypes.h"
#include "HAL/Runnable.h"

#define UE_API MEDIAIOCORE_API

/**
* Implementation of WaitSyncThread
*/

class IMediaIOCoreHardwareSync;

class FMediaIOCoreWaitVSyncThread : public FRunnable
{
public:
	UE_API FMediaIOCoreWaitVSyncThread(TSharedPtr<IMediaIOCoreHardwareSync> InHardwareSync);

	UE_API virtual bool Init() override;
	UE_API virtual uint32 Run() override;

	UE_API virtual void Stop() override;
	UE_API virtual void Exit() override;

public:
	UE_API bool Wait_GameOrRenderThread();

protected:
	TSharedPtr<IMediaIOCoreHardwareSync> HardwareSync;
	FEvent* WaitVSync;
	TAtomic<bool> bWaitingForSignal;
	TAtomic<bool> bAlive;
};

#undef UE_API
