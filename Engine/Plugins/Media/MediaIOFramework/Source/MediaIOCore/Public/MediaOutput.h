// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "PixelFormat.h"

#include "MediaOutput.generated.h"

#define UE_API MEDIAIOCORE_API


class UMediaCapture;

enum class EMediaCaptureConversionOperation : int8
{
	NONE,
	RGBA8_TO_YUV_8BIT,
	RGB10_TO_YUVv210_10BIT,
	INVERT_ALPHA,
	SET_ALPHA_ONE,
	CUSTOM, //~ If Custom used, OnCustomCapture_RenderingThread, GetCustomOutputSize and GetCustomOutputPixelFormat on the MediaCapture need to be implemented
};

UENUM()
enum class EMediaCaptureSourceType : int8
{
	RENDER_TARGET,
	SCENE_VIEWPORT,
	RHI_RESOURCE,
	MEDIA_TEXTURE
};

/**
 * Abstract base class for media output.
 *
 * Media output describe the location and/or settings of media objects that can
 * be used to output the content of Unreal Engine to a target device via a MediaCapture.
 */
UCLASS(MinimalAPI, Abstract, editinlinenew, BlueprintType, hidecategories = (Object))
class UMediaOutput : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	/**
	 * Number of texture used to transfer the texture from the GPU to the system memory.
	 * A smaller number is most likely to block the GPU (wait for the transfer to complete).
	 * A bigger number is most likely to increase latency.
	 * @note Some Capture are not are executed on the GPU. If it's the case then no buffer will be needed and no buffer will be created.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category="Output", meta=(ClampMin=1, ClampMax=8))
	int32 NumberOfTextureBuffers;

	/** Creates the specific implementation of the MediaCapture for the MediaOutput. */
	UFUNCTION(BlueprintCallable, Category="Media|Output")
	UE_API UMediaCapture* CreateMediaCapture();

	/**
	 * Validate the media output settings (must be implemented in child classes).
	 *
	 * @return true if validation passed, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|Output")
	UE_API virtual bool Validate(FString& OutFailureReason) const;

public:

	static UE_API const FIntPoint RequestCaptureSourceSize;

	/**
	 * The size of the buffer we wish to capture.
	 * The size of the buffer can not change during the capture.
	 * Return UMediaOutput::RequestCaptureSourceSize if you wish to take the buffer size as the requested size.
	 */
	virtual FIntPoint GetRequestedSize() const PURE_VIRTUAL(UMediaOutput::GetRequestedSize, return RequestCaptureSourceSize; );

	/**
	 * The pixel format of the buffer we wish to capture.
	 * Some conversion are available. See EMediaCaptureConversionOperation
	 */
	virtual EPixelFormat GetRequestedPixelFormat() const PURE_VIRTUAL(UMediaOutput::GetRequestedPixelFormat, return EPixelFormat::PF_Unknown; );

	/**
	 * The conversion we wish to accomplish on the GPU before the DMA transfer occurs.
	 */
	virtual EMediaCaptureConversionOperation GetConversionOperation(EMediaCaptureSourceType InSourceType) const { return EMediaCaptureConversionOperation::NONE; }
	
#if WITH_EDITOR
	/** Gets a string that sums up the media output's description for UI purposes */
	virtual FString GetDescriptionString() const { return FString(); }

	typedef TTuple<FText, FText, FText> FInfoElement;
	
	/**
	 * Gets a list of text containing the media source's configuration and description for UI purposes
	 * @param OutInfoElements A list of elements, where an element is constructed from a group, a label, and a value
	 */
	virtual void GetDetailsPanelInfoElements(TArray<FInfoElement>& OutInfoElements) const { }
	
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnOutputModified, UMediaOutput*);
	FOnOutputModified& OnOutputModified() { return OnOutputModifiedDelegate; }
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

protected:
	virtual UMediaCapture* CreateMediaCaptureImpl() { return nullptr; }

private:
#if WITH_EDITOR
	FOnOutputModified OnOutputModifiedDelegate;
#endif

};

#undef UE_API
