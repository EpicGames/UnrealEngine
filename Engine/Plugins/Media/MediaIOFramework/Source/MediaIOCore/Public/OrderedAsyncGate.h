// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericPlatformProcess.h"
#include "HAL/PlatformProcess.h"
#include <atomic>

namespace UE::MediaIO
{
	/*
	 * Class to ensure that async tasks execute any critical path in order.
	 * It works like clients at a shop getting a number, waiting for their turn, and
	 * when they are done they give up their turn so that the next number can be serviced.
	 * Its functions can be called from any thread.
	 */
	class FOrderedAsyncGate
	{
	public:

		/** Gets a number that represents the when the number holder will get their turn. */
		uint64 GetANumber()
		{
			return NextNumber++;
		}

		/** Returns true if it is the turn of the given number */
		bool IsMyTurn(const uint32 Number) const
		{
			return Number == CurrentNumber;
		}

		/**
		 * This function will block until it is the turn of the given number to execute.
		 * It can be called multiple times until the turn is given up by calling GiveUpTurn.
		 * The number given must have been received from a call to GetANumber.
		 */
		void WaitForTurn(const uint32 Number) const
		{
			while (!IsMyTurn(Number))
			{
				constexpr float SpinWaitTimeSeconds = 50e-6f;

				FPlatformProcess::SleepNoStats(SpinWaitTimeSeconds);
			}
		}

		/**
		 * Call this function when you are ready to give up your turn.
		 * It must be called with the number received from GetANumber, and it must
		 * be called whether WaitForTurn was called or not. Not giving up the turn
		 * will prevent any others from getting their turn.
		 * Note: It will internally call WaitForTurn.
		 */
		void GiveUpTurn(const uint32 Number)
		{
			WaitForTurn(Number);

			++CurrentNumber;
		}

	private:

		/** The current number that has its turn without having to wait any longer */
		std::atomic<uint32> CurrentNumber = 0;

		/** The number that will be given out to the next client that calls GetANumber */
		std::atomic<uint32> NextNumber = 0;
	};

}
