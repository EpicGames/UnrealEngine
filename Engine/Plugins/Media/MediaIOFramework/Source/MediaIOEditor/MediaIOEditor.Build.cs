// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MediaIOEditor : ModuleRules
	{
		public MediaIOEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"EditorFramework",
					"MediaIOCore",
					"Slate",
					"SlateCore",
					"TimeManagement",
					"UnrealEd",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetDefinition",
					"Engine",
					"PropertyEditor",
					"MediaAssets"
				});
		}
	}
}
