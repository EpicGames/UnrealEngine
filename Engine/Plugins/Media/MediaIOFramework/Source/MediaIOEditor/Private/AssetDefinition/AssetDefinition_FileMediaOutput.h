// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "FileMediaOutput.h"
#include "AssetDefinition/AssetDefinition_MediaOutput.h"
#include "AssetDefinition_FileMediaOutput.generated.h"

UCLASS()
class UAssetDefinition_FileMediaOutput : public UAssetDefinition_MediaOutput
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_FileMediaOutput", "File Media Output"); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UFileMediaOutput::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Media, NSLOCTEXT("AssetDefinition", "AssetDefinition_FileMediaOutputSubMenu", "Media Sources + Outputs"), ECategoryMenuType::Section) };
		return Categories;
	}
	// UAssetDefinition End
};
