// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition/AssetDefinition_MediaOutput.h"
#include "MediaOutput.h"

FLinearColor UAssetDefinition_MediaOutput::GetAssetColor() const
{
	return FColor::White;
}

TSoftClassPtr<UObject> UAssetDefinition_MediaOutput::GetAssetClass() const
{
	return UMediaOutput::StaticClass();
}

bool UAssetDefinition_MediaOutput::CanImport() const
{
	return false;
}
