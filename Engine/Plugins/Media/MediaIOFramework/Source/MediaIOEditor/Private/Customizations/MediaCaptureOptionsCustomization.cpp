// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/MediaCaptureOptionsCustomization.h"

#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "MediaCapture.h"
#include "PropertyHandle.h"
#include "PropertyRestriction.h"

#define LOCTEXT_NAMESPACE "MediaCaptureOptionsCustomization"

TSharedRef<IPropertyTypeCustomization> FMediaCaptureOptionsCustomization::MakeInstance()
{
	return MakeShareable(new FMediaCaptureOptionsCustomization);
}

void FMediaCaptureOptionsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	HeaderRow.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	];
}

void FMediaCaptureOptionsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	CapturePhaseHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMediaCaptureOptions, CapturePhase));
	ForceAlphaHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMediaCaptureOptions, bForceAlphaToOneOnConversion));

	if (CapturePhaseHandle.IsValid())
	{
		CapturePhaseHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FMediaCaptureOptionsCustomization::OnCapturePhaseChanged));

		// BackBufferReady captures the entire back buffer including editor UI and is not meaningful in editor context.
		// Hide it from the dropdown; it remains available via code for runtime-only use cases.
		TSharedRef<FPropertyRestriction> Restriction = MakeShared<FPropertyRestriction>(
			LOCTEXT("BackBufferReadyEditorRestriction", "Not available in the editor. Use via code for runtime-only captures."));
		const UEnum* CapturePhaseEnum = StaticEnum<EMediaCapturePhase>();
		if (CapturePhaseEnum)
		{
			Restriction->AddHiddenValue(CapturePhaseEnum->GetNameStringByValue((int64)EMediaCapturePhase::BackBufferReady));
		}
		CapturePhaseHandle->AddRestriction(Restriction);
	}

	uint32 NumChildren = 0;
	PropertyHandle->GetNumChildren(NumChildren);
	for (uint32 ChildIdx = 0; ChildIdx < NumChildren; ++ChildIdx)
	{
		TSharedPtr<IPropertyHandle> ChildHandle = PropertyHandle->GetChildHandle(ChildIdx);
		if (!ChildHandle.IsValid())
		{
			continue;
		}

		IDetailPropertyRow& Row = ChildBuilder.AddProperty(ChildHandle.ToSharedRef());

		if (ForceAlphaHandle.IsValid() && ChildHandle->GetProperty() == ForceAlphaHandle->GetProperty())
		{
			Row.IsEnabled(TAttribute<bool>::CreateSP(this, &FMediaCaptureOptionsCustomization::IsForceAlphaEditable));
		}
	}
}

void FMediaCaptureOptionsCustomization::OnCapturePhaseChanged()
{
	if (!CapturePhaseHandle.IsValid() || !ForceAlphaHandle.IsValid())
	{
		return;
	}

	uint8 PhaseValue = 0;
	if (CapturePhaseHandle->GetValue(PhaseValue) == FPropertyAccess::Success)
	{
		if (static_cast<EMediaCapturePhase>(PhaseValue) == EMediaCapturePhase::BackBufferReady)
		{
			ForceAlphaHandle->SetValue(true);
		}
	}
}

bool FMediaCaptureOptionsCustomization::IsForceAlphaEditable() const
{
	if (!CapturePhaseHandle.IsValid())
	{
		return true;
	}

	uint8 PhaseValue = 0;
	if (CapturePhaseHandle->GetValue(PhaseValue) == FPropertyAccess::Success)
	{
		return static_cast<EMediaCapturePhase>(PhaseValue) != EMediaCapturePhase::BackBufferReady;
	}
	return true;
}

#undef LOCTEXT_NAMESPACE
