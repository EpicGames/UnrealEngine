// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "PropertyHandle.h"

/**
 * Implements a details view customization for FMediaCaptureOptions.
 * Automatically enables bForceAlphaToOneOnConversion when BackBufferReady capture phase is selected,
 * and makes the option read-only while that phase is active.
 */
class FMediaCaptureOptionsCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:
	/** Called when CapturePhase changes; auto-sets bForceAlphaToOneOnConversion to true when BackBufferReady is selected. */
	void OnCapturePhaseChanged();

	/** Returns false when BackBufferReady is the active capture phase, making the force-alpha option read-only. */
	bool IsForceAlphaEditable() const;

	/** Handle to the CapturePhase property; watched to drive the alpha-forcing behavior. */
	TSharedPtr<IPropertyHandle> CapturePhaseHandle;

	/** Handle to the bForceAlphaToOneOnConversion property; toggled and gated based on the selected capture phase. */
	TSharedPtr<IPropertyHandle> ForceAlphaHandle;
};
