// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/MediaIOConfigurationCustomization.h"

#include "Algo/Transform.h"
#include "DetailWidgetRow.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "GenlockedCustomTimeStep.h"
#include "IDetailChildrenBuilder.h"
#include "IMediaIOCoreDeviceProvider.h"
#include "IMediaIOCoreModule.h"
#include "MediaIOPermutationsSelectorBuilder.h"
#include "ObjectEditorUtils.h"
#include "ScopedTransaction.h"
#include "TimeSynchronizableMediaSource.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SMediaPermutationsSelector.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWindow.h"

#define LOCTEXT_NAMESPACE "MediaIOConfigurationCustomization"

TSharedRef<IPropertyTypeCustomization> FMediaIOConfigurationCustomization::MakeInstance()
{
	return MakeShared<FMediaIOConfigurationCustomization>();
}

void FMediaIOConfigurationCustomization::OnCustomizeObjects(const TArray<UObject*>& InCustomizedObjects)
{
	Objects.Reserve(InCustomizedObjects.Num());
	Algo::Transform(InCustomizedObjects, Objects, [](UObject* Object) { return TWeakObjectPtr<>{ Object }; });
}

TAttribute<FText> FMediaIOConfigurationCustomization::GetContentText()
{
	FMediaIOConfiguration* Value = GetPropertyValueFromPropertyHandle<FMediaIOConfiguration>();
	IMediaIOCoreDeviceProvider* DeviceProviderPtr = IMediaIOCoreModule::Get().GetDeviceProvider(DeviceProviderName);
	if (DeviceProviderPtr)
	{
		return MakeAttributeLambda([this, DeviceProviderPtr, Value]
			{ 
				return DeviceProviderPtr->ToText(*Value, IsAutoDetected()); 
			});
	}
	return FText::GetEmpty();
}

TSharedRef<SWidget> FMediaIOConfigurationCustomization::HandleSourceComboButtonMenuContent()
{
	PermutationSelector.Reset();

	IMediaIOCoreDeviceProvider* DeviceProviderPtr = IMediaIOCoreModule::Get().GetDeviceProvider(DeviceProviderName);
	if (DeviceProviderPtr == nullptr)
	{
		return SNew(STextBlock)
			.Text(LOCTEXT("NoDeviceProviderFound", "No provider found"));
	}

	SelectedConfiguration = *GetPropertyValueFromPropertyHandle<FMediaIOConfiguration>();
	bool bIsInput = SelectedConfiguration.bIsInput;
	if (!SelectedConfiguration.IsValid())
	{
		SelectedConfiguration = DeviceProviderPtr->GetDefaultConfiguration();
		SelectedConfiguration.bIsInput = bIsInput;
	}

	bAutoDetectFormat = IsAutoDetected();

	TArray<FMediaIOConfiguration> MediaConfigurations = bIsInput ? DeviceProviderPtr->GetConfigurations(true, false) : DeviceProviderPtr->GetConfigurations(false, true);
	if (MediaConfigurations.Num() == 0)
	{
		return SNew(STextBlock)
			.Text(LOCTEXT("NoConfigurationFound", "No configuration found"));
	}

	auto QuadTypeVisible = [](FName ColumnName, const TArray<FMediaIOConfiguration>& UniquePermutationsForThisColumn)
	{
		if (UniquePermutationsForThisColumn.Num() > 0)
		{
			return UniquePermutationsForThisColumn[0].MediaConnection.TransportType == EMediaIOTransportType::QuadLink;
		}
		return false;
	};

	TSharedRef<SWidget> AutoCheckbox = SNullWidget::NullWidget;

	if (bIsInput)
	{
		bool bSupportsAutoDetect = true;
		for (UObject* Object : GetCustomizedObjects())
		{
			if (!Object)
			{
				bSupportsAutoDetect = false;
				break;
			}
			
			if (UTimeSynchronizableMediaSource* MediaSource = Cast<UTimeSynchronizableMediaSource>(Object))
			{
				bSupportsAutoDetect &= MediaSource->SupportsFormatAutoDetection();
			}
			else if (UGenlockedCustomTimeStep* CustomTimeStep = Cast<UGenlockedCustomTimeStep>(Object))
			{
				bSupportsAutoDetect &= CustomTimeStep->SupportsFormatAutoDetection();
			}
			else
			{
				bSupportsAutoDetect = false;
			}
		}

		if (bSupportsAutoDetect)
		{
			AutoCheckbox = SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(4.f)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(NSLOCTEXT("MediaPlayerEditor", "AutoLabel", "Auto"))
				]
			+ SHorizontalBox::Slot()
				.Padding(4.f)
				.AutoWidth()
				[
					SNew(SCheckBox)
					.ToolTipText(NSLOCTEXT("MediaPlayerEditor", "AutoToolTip", "Automatically detect the video format of the signal coming through this input. When disabled, the format will be enforced and the source will error out if the format differs from the expected one."))
					.IsChecked(this, &FMediaIOConfigurationCustomization::GetAutoCheckboxState)
					.OnCheckStateChanged(this, &FMediaIOConfigurationCustomization::SetAutoCheckboxState)
				];
		}		

	}

	auto GetExtensions = [AutoCheckbox](TArray<TSharedRef<SWidget>>& OutWidgets)
	{
		if (AutoCheckbox != SNullWidget::NullWidget)
		{
			OutWidgets.Add(AutoCheckbox);
		}
	};

	using TSelection = SMediaPermutationsSelector<FMediaIOConfiguration, FMediaIOPermutationsSelectorBuilder>;
	TSelection::FArguments Arguments;
	Arguments
		.PermutationsSource(MoveTemp(MediaConfigurations))
		.SelectedPermutation(SelectedConfiguration)
		.OnSelectionChanged(this, &FMediaIOConfigurationCustomization::OnSelectionChanged)
		.OnButtonClicked(this, &FMediaIOConfigurationCustomization::OnButtonClicked)
		.OnGetExtensions_Lambda(GetExtensions)
		+ TSelection::Column(FMediaIOPermutationsSelectorBuilder::NAME_DeviceIdentifier)
		.Label(LOCTEXT("DeviceLabel", "Device"));

	if (DeviceProviderPtr->ShowInputTransportInSelector())
	{
		Arguments
			+ TSelection::Column(FMediaIOPermutationsSelectorBuilder::NAME_TransportType)
			.Label(LOCTEXT("SourceTypeLabel", "Source"))
			+TSelection::Column(FMediaIOPermutationsSelectorBuilder::NAME_QuadType)
			.Label(LOCTEXT("QuadTypeLabel", "Quad"))
			.IsColumnVisible_Lambda(QuadTypeVisible);
	}

	Arguments
		+ TSelection::Column(FMediaIOPermutationsSelectorBuilder::NAME_Resolution)
		.Label(LOCTEXT("ResolutionLabel", "Resolution"))
		.IsColumnVisible_Raw(this, &FMediaIOConfigurationCustomization::ShowAdvancedColumns)
		+ TSelection::Column(FMediaIOPermutationsSelectorBuilder::NAME_Standard)
		.Label(LOCTEXT("StandardLabel", "Standard"))
		.IsColumnVisible_Raw(this, &FMediaIOConfigurationCustomization::ShowAdvancedColumns)
		+ TSelection::Column(FMediaIOPermutationsSelectorBuilder::NAME_FrameRate)
		.Label(LOCTEXT("FrameRateLabel", "Frame Rate"))
		.IsColumnVisible_Raw(this, &FMediaIOConfigurationCustomization::ShowAdvancedColumns);



	TSharedRef<TSelection> Selector = SNew(TSelection) = Arguments;
	PermutationSelector = Selector;
	SelectedConfiguration = Selector->GetSelectedItem();

	return Selector;
}

void FMediaIOConfigurationCustomization::OnSelectionChanged(FMediaIOConfiguration SelectedItem)
{
	SelectedConfiguration = SelectedItem;
}

FReply FMediaIOConfigurationCustomization::OnButtonClicked()
{
	AssignValue(SelectedConfiguration);
	// Make sure to overwrite what was in the config since the auto value is determined by the timecode provider and not the generated configs.
	SetIsAutoDetected(bAutoDetectFormat);

	TSharedPtr<SWidget> SharedPermutationSelector = PermutationSelector.Pin();
	if (SharedPermutationSelector.IsValid())
	{
		TSharedRef<SWindow> ParentContextMenuWindow = FSlateApplication::Get().FindWidgetWindow(SharedPermutationSelector.ToSharedRef()).ToSharedRef();
		FSlateApplication::Get().RequestDestroyWindow(ParentContextMenuWindow);
	}

	return FReply::Handled();
}

ECheckBoxState FMediaIOConfigurationCustomization::GetAutoCheckboxState() const
{
	return bAutoDetectFormat ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FMediaIOConfigurationCustomization::SetAutoCheckboxState(ECheckBoxState CheckboxState)
{
	bAutoDetectFormat = CheckboxState == ECheckBoxState::Checked;
}

bool FMediaIOConfigurationCustomization::ShowAdvancedColumns(FName ColumnName, const TArray<FMediaIOConfiguration>& UniquePermutationsForThisColumn) const
{
	return !bAutoDetectFormat;
}

bool FMediaIOConfigurationCustomization::IsAutoDetected() const
{
	bool bAutoDetectTimecode = true;

	for (const TWeakObjectPtr<UObject>& WeakObject : Objects)
	{
		if (TStrongObjectPtr<UObject> CustomizedObject = WeakObject.Pin())
		{
			if (const UTimeSynchronizableMediaSource* Source = Cast<UTimeSynchronizableMediaSource>(CustomizedObject.Get()))
			{
				if (!Source->bAutoDetectInput)
				{
					bAutoDetectTimecode = false;
					break;
				}
			}

			if (const UGenlockedCustomTimeStep* CustomTimeStep = Cast<UGenlockedCustomTimeStep>(CustomizedObject.Get()))
			{
				if (!CustomTimeStep->bAutoDetectFormat)
				{
					bAutoDetectTimecode = false;
					break;
				}
			}
		}
	}

	return bAutoDetectTimecode;
}

void FMediaIOConfigurationCustomization::SetIsAutoDetected(bool Value)
{
	FScopedTransaction Transaction{ LOCTEXT("MediaIOAutoDetectTimecode", "Auto Detect Timecode") };

	for (const TWeakObjectPtr<UObject>& WeakObject : Objects)
	{
		if (TStrongObjectPtr<UObject> CustomizedObject = WeakObject.Pin())
		{
			if (UTimeSynchronizableMediaSource* Source = Cast<UTimeSynchronizableMediaSource>(CustomizedObject.Get()))
			{
				Source->Modify();
				Source->bAutoDetectInput = Value;
			}

			if (UGenlockedCustomTimeStep* CustomTimeStep = Cast<UGenlockedCustomTimeStep>(CustomizedObject.Get()))
			{
				CustomTimeStep->Modify();
				CustomTimeStep->bAutoDetectFormat = Value;
			}
		}
	}
}


#undef LOCTEXT_NAMESPACE
