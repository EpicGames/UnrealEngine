// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/MediaIOCustomizationBase.h"

#include "MediaIOCoreDefinitions.h"
#include "DetailWidgetRow.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "IPropertyUtilities.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "MediaIOCustomizationBase"

namespace FMediaIOCustomizationBaseHelper
{
	FName TraverseClassHierarchy(UClass* InClass, const FName& InMetadataName)
	{
		if (InClass == nullptr)
		{
			return NAME_None;
		}

		if (!InClass->HasMetaData(InMetadataName))
		{
			return TraverseClassHierarchy(InClass->GetSuperClass(), InMetadataName);
		}

		return *InClass->GetMetaData(InMetadataName);
	}
}

void FMediaIOCustomizationBase::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	MediaProperty = InPropertyHandle;

	// Read the media provider name
	if (MediaProperty->IsValidHandle())
	{
		static FName NAME_MediaIOCustomLayout = TEXT("MediaIOCustomLayout");
		// FProperties: it doesn't make sense to ask for MetaData of a property class as it's always native and doesn't have metadata
		//if (MediaProperty->HasMetaData(NAME_MediaIOCustomLayout))
		//{			
			//DeviceProviderName = *MediaProperty->GetPropertyClass()->GetMetaData(NAME_MediaIOCustomLayout);
		//}

		TArray<UObject*> OuterObjects;
		MediaProperty->GetOuterObjects(OuterObjects);

		// Call into child classes as soon as we have access to the uobjects.
		OnCustomizeObjects(OuterObjects);

		if (DeviceProviderName == NAME_None)
		{
			for (UObject* Obj : OuterObjects)
			{
				FName FoundMetadata = FMediaIOCustomizationBaseHelper::TraverseClassHierarchy(Obj->GetClass(), NAME_MediaIOCustomLayout);
				if (FoundMetadata == NAME_None)
				{
					DeviceProviderName = NAME_None;
					break;
				}

				if (DeviceProviderName == NAME_None)
				{
					DeviceProviderName = FoundMetadata;
				}
				else if (DeviceProviderName != FoundMetadata)
				{
					DeviceProviderName = NAME_None;
					break;
				}
			}
		}
		ensure(DeviceProviderName != NAME_None);
	}

	if (MediaProperty->GetNumPerObjectValues() == 1 && MediaProperty->IsValidHandle())
	{
		TSharedPtr<IPropertyUtilities> PropertyUtils = CustomizationUtils.GetPropertyUtilities();

		HeaderRow
		.NameContent()
		[
			InPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MaxDesiredWidth(512)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(GetContentText())
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
			.VAlign(VAlign_Center)
			[
				SNew(SComboButton)
				.OnGetMenuContent_Lambda( [this]() { return HandleSourceComboButtonMenuContent(); } )
				.ContentPadding(FMargin(4.0, 2.0))
			]
		].IsEnabled(MakeAttributeLambda([=] { return !InPropertyHandle->IsEditConst() && PropertyUtils->IsPropertyEditingEnabled(); }));
	}
}

void FMediaIOCustomizationBase::CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumberOfChild;
	if (InStructPropertyHandle->GetNumChildren(NumberOfChild) == FPropertyAccess::Success)
	{
		for (uint32 Index = 0; Index < NumberOfChild; ++Index)
		{
			StructBuilder.AddProperty(InStructPropertyHandle->GetChildHandle(Index).ToSharedRef()).IsEnabled(false).ShowPropertyButtons(false);
		}
	}
}

TArray<UObject*> FMediaIOCustomizationBase::GetCustomizedObjects() const
{
	TArray<UObject*> Objects;
	if (MediaProperty)
	{
		MediaProperty->GetOuterObjects(Objects);
	}
	return Objects;
}

void FMediaIOCustomizationBase::AssignValueImpl(const void* NewValue) const
{
	if (FStructProperty* StructProperty = CastField<FStructProperty>(MediaProperty->GetProperty()))
	{
		TArray<void*> RawData;
		MediaProperty->AccessRawData(RawData);

		FString TextValue;
		StructProperty->Struct->ExportText(TextValue, NewValue, RawData[0], nullptr, EPropertyPortFlags::PPF_None, nullptr);
		ensure(MediaProperty->SetValueFromFormattedString(TextValue, EPropertyValueSetFlags::DefaultFlags) == FPropertyAccess::Result::Success);
	}
}

#undef LOCTEXT_NAMESPACE
