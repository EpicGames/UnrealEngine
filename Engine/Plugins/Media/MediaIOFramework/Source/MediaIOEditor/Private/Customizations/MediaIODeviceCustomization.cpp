// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/MediaIODeviceCustomization.h"

#include "DetailWidgetRow.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IMediaIOCoreDeviceProvider.h"
#include "IMediaIOCoreModule.h"
#include "IPropertyUtilities.h"
#include "MediaIOCoreDefinitions.h"
#include "PropertyHandle.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "MediaIODeviceCustomization"

TSharedRef<IPropertyTypeCustomization> FMediaIODeviceCustomization::MakeInstance()
{
	TSharedRef<FMediaIODeviceCustomization> Result = MakeShareable(new FMediaIODeviceCustomization);
	return Result;
}

TAttribute<FText> FMediaIODeviceCustomization::GetContentText()
{
	FMediaIODevice* Value = GetPropertyValueFromPropertyHandle<FMediaIODevice>();
	IMediaIOCoreDeviceProvider* DeviceProviderPtr = IMediaIOCoreModule::Get().GetDeviceProvider(DeviceProviderName);
	if (DeviceProviderPtr)
	{
		return MakeAttributeLambda([=] { return DeviceProviderPtr->ToText(*Value); });
	}
	return FText::GetEmpty();
}

TSharedRef<SWidget> FMediaIODeviceCustomization::HandleSourceComboButtonMenuContent()
{
	IMediaIOCoreDeviceProvider* DeviceProviderPtr = IMediaIOCoreModule::Get().GetDeviceProvider(DeviceProviderName);
	if (DeviceProviderPtr == nullptr)
	{
		return SNew(STextBlock)
			.Text(LOCTEXT("NoDeviceProviderFound", "No provider found"));
	}

	// found all devices
	TArray<FMediaIODevice> AllDevices = DeviceProviderPtr->GetDevices();
	if (AllDevices.Num() == 0)
	{
		return SNew(STextBlock)
			.Text(LOCTEXT("NoDeviceFound", "No device found"));
	}

	// generate menu
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.BeginSection("AllDevices", LOCTEXT("AllDevicesSection", "All Devices"));
	{
		for (const FMediaIODevice& Device : AllDevices)
		{
			MenuBuilder.AddMenuEntry(
				DeviceProviderPtr->ToText(Device),
				DeviceProviderPtr->ToText(Device),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([this, Device] {
						AssignValue(Device);
					}),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this, Device] {
						TArray<void*> RawData;
						GetMediaProperty()->AccessRawData(RawData);
						FMediaIODevice* PreviousDevice = reinterpret_cast<FMediaIODevice*>(RawData[0]);
						return *PreviousDevice == Device;
					})
				),
				NAME_None,
				EUserInterfaceActionType::RadioButton
				);
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE
