// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/MediaIOInputConfigurationCustomization.h"

#include "DetailWidgetRow.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "IDetailChildrenBuilder.h"
#include "IMediaIOCoreDeviceProvider.h"
#include "IMediaIOCoreModule.h"
#include "MediaIOPermutationsSelectorBuilder.h"
#include "ObjectEditorUtils.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SMediaPermutationsSelector.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWindow.h"

#define LOCTEXT_NAMESPACE "MediaIOInputConfigurationCustomization"

TSharedRef<IPropertyTypeCustomization> FMediaIOInputConfigurationCustomization::MakeInstance()
{
	return MakeShareable(new FMediaIOInputConfigurationCustomization);
}

TAttribute<FText> FMediaIOInputConfigurationCustomization::GetContentText()
{
	FMediaIOInputConfiguration* Value = GetPropertyValueFromPropertyHandle<FMediaIOInputConfiguration>();
	IMediaIOCoreDeviceProvider* DeviceProviderPtr = IMediaIOCoreModule::Get().GetDeviceProvider(DeviceProviderName);
	if (DeviceProviderPtr)
	{
		return MakeAttributeLambda([=] { return DeviceProviderPtr->ToText(*Value); });
	}
	return FText::GetEmpty();
}

TSharedRef<SWidget> FMediaIOInputConfigurationCustomization::HandleSourceComboButtonMenuContent()
{
	PermutationSelector.Reset();

	IMediaIOCoreDeviceProvider* DeviceProviderPtr = IMediaIOCoreModule::Get().GetDeviceProvider(DeviceProviderName);
	if (DeviceProviderPtr == nullptr)
	{
		return SNew(STextBlock)
			.Text(LOCTEXT("NoDeviceProviderFound", "No provider found"));
	}

	SelectedConfiguration = *GetPropertyValueFromPropertyHandle<FMediaIOInputConfiguration>();
	if (!SelectedConfiguration.IsValid())
	{
		SelectedConfiguration = DeviceProviderPtr->GetDefaultInputConfiguration();
	}


	TArray<FMediaIOInputConfiguration> MediaConfigurations = DeviceProviderPtr->GetInputConfigurations();
	if (MediaConfigurations.Num() == 0)
	{
		return SNew(STextBlock)
			.Text(LOCTEXT("NoConfigurationFound", "No configuration found"));
	}

	// Create the permutation selector
	auto QuadTypeVisible = [](FName ColumnName, const TArray<FMediaIOInputConfiguration>& UniquePermutationsForThisColumn)
	{
		if (UniquePermutationsForThisColumn.Num() > 0)
		{
			return UniquePermutationsForThisColumn[0].MediaConfiguration.MediaConnection.TransportType == EMediaIOTransportType::QuadLink;
		}
		return false;
	};

	auto KeyVisible = [](FName ColumnName, const TArray<FMediaIOInputConfiguration>& UniquePermutationsForThisColumn)
	{
		if (UniquePermutationsForThisColumn.Num() > 0)
		{
			return UniquePermutationsForThisColumn[0].InputType == EMediaIOInputType::FillAndKey;
		}
		return false;
	};

	using TSelection = SMediaPermutationsSelector<FMediaIOInputConfiguration, FMediaIOPermutationsSelectorBuilder>;
	TSelection::FArguments Arguments;
	Arguments
		.PermutationsSource(MoveTemp(MediaConfigurations))
		.SelectedPermutation(SelectedConfiguration)
		.OnSelectionChanged(this, &FMediaIOInputConfigurationCustomization::OnSelectionChanged)
		.OnButtonClicked(this, &FMediaIOInputConfigurationCustomization::OnButtonClicked)
		+ TSelection::Column(FMediaIOPermutationsSelectorBuilder::NAME_InputType)
		.Label(LOCTEXT("InputTypeeLabel", "Input Type"))
		+ TSelection::Column(FMediaIOPermutationsSelectorBuilder::NAME_DeviceIdentifier)
		.Label(LOCTEXT("DeviceLabel", "Device"));

	if (DeviceProviderPtr->ShowInputTransportInSelector())
	{
		Arguments
			+ TSelection::Column(FMediaIOPermutationsSelectorBuilder::NAME_TransportType)
			.Label(LOCTEXT("SourceTypeLabel", "Source"))
			+ TSelection::Column(FMediaIOPermutationsSelectorBuilder::NAME_QuadType)
			.Label(LOCTEXT("QuadTypeLabel", "Quad"))
			.IsColumnVisible_Lambda(QuadTypeVisible);
	}

	Arguments
		+ TSelection::Column(FMediaIOPermutationsSelectorBuilder::NAME_Resolution)
		.Label(LOCTEXT("ResolutionLabel", "Resolution"))
		+ TSelection::Column(FMediaIOPermutationsSelectorBuilder::NAME_Standard)
		.Label(LOCTEXT("StandardLabel", "Standard"))
		+ TSelection::Column(FMediaIOPermutationsSelectorBuilder::NAME_FrameRate)
		.Label(LOCTEXT("FrameRateLabel", "Frame Rate"));

	if (DeviceProviderPtr->ShowInputKeyInSelector())
	{
		Arguments
			+ TSelection::Column(FMediaIOPermutationsSelectorBuilder::NAME_KeyPortSource)
			.Label(LOCTEXT("KeySourceTypeLabel", "Key Source"))
			.IsColumnVisible_Lambda(KeyVisible);
	}

	TSharedRef<TSelection> Selector = SNew(TSelection) = Arguments;
	PermutationSelector = Selector;
	SelectedConfiguration = Selector->GetSelectedItem();

	return Selector;
}

void FMediaIOInputConfigurationCustomization::OnSelectionChanged(FMediaIOInputConfiguration SelectedItem)
{
	SelectedConfiguration = SelectedItem;
}

FReply FMediaIOInputConfigurationCustomization::OnButtonClicked() const
{
	AssignValue(SelectedConfiguration);

	TSharedPtr<SWidget> SharedPermutationSelector = PermutationSelector.Pin();
	if (SharedPermutationSelector.IsValid())
	{
		TSharedRef<SWindow> ParentContextMenuWindow = FSlateApplication::Get().FindWidgetWindow(SharedPermutationSelector.ToSharedRef()).ToSharedRef();
		FSlateApplication::Get().RequestDestroyWindow(ParentContextMenuWindow);
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
