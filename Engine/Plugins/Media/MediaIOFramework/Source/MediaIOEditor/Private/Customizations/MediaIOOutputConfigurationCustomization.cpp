// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/MediaIOOutputConfigurationCustomization.h"

#include "DetailWidgetRow.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "IDetailChildrenBuilder.h"
#include "IMediaIOCoreDeviceProvider.h"
#include "IMediaIOCoreModule.h"
#include "MediaIOPermutationsSelectorBuilder.h"
#include "ObjectEditorUtils.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SMediaPermutationsSelector.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWindow.h"

#define LOCTEXT_NAMESPACE "MediaIOOutputConfigurationCustomization"

TSharedRef<IPropertyTypeCustomization> FMediaIOOutputConfigurationCustomization::MakeInstance()
{
	return MakeShareable(new FMediaIOOutputConfigurationCustomization);
}

TAttribute<FText> FMediaIOOutputConfigurationCustomization::GetContentText()
{
	FMediaIOOutputConfiguration* Value = GetPropertyValueFromPropertyHandle<FMediaIOOutputConfiguration>();
	IMediaIOCoreDeviceProvider* DeviceProviderPtr = IMediaIOCoreModule::Get().GetDeviceProvider(DeviceProviderName);
	if (DeviceProviderPtr)
	{
		return MakeAttributeLambda([=] { return DeviceProviderPtr->ToText(*Value); });
	}
	return FText::GetEmpty();
}

TSharedRef<SWidget> FMediaIOOutputConfigurationCustomization::HandleSourceComboButtonMenuContent()
{
	PermutationSelector.Reset();

	IMediaIOCoreDeviceProvider* DeviceProviderPtr = IMediaIOCoreModule::Get().GetDeviceProvider(DeviceProviderName);
	if (DeviceProviderPtr == nullptr)
	{
		return SNew(STextBlock)
			.Text(LOCTEXT("NoDeviceProviderFound", "No provider found"));
	}

	SelectedConfiguration = *GetPropertyValueFromPropertyHandle<FMediaIOOutputConfiguration>();
	if (!SelectedConfiguration.IsValid())
	{
		SelectedConfiguration = DeviceProviderPtr->GetDefaultOutputConfiguration();
	}


	TArray<FMediaIOOutputConfiguration> MediaConfigurations = DeviceProviderPtr->GetOutputConfigurations();
	if (MediaConfigurations.Num() == 0)
	{
		return SNew(STextBlock)
			.Text(LOCTEXT("NoConfigurationFound", "No configuration found"));
	}

	// Create the permutation selector
	auto QuadTypeVisible = [](FName ColumnName, const TArray<FMediaIOOutputConfiguration>& UniquePermutationsForThisColumn)
	{
		if (UniquePermutationsForThisColumn.Num() > 0)
		{
			return UniquePermutationsForThisColumn[0].MediaConfiguration.MediaConnection.TransportType == EMediaIOTransportType::QuadLink;
		}
		return false;
	};

	auto KeyVisible = [](FName ColumnName, const TArray<FMediaIOOutputConfiguration>& UniquePermutationsForThisColumn)
	{
		if (UniquePermutationsForThisColumn.Num() > 0)
		{
			return UniquePermutationsForThisColumn[0].OutputType == EMediaIOOutputType::FillAndKey;
		}
		return false;
	};

	auto SyncVisible = [](FName ColumnName, const TArray<FMediaIOOutputConfiguration>& UniquePermutationsForThisColumn)
	{
		if (UniquePermutationsForThisColumn.Num() > 0)
		{
			return UniquePermutationsForThisColumn[0].OutputReference == EMediaIOReferenceType::Input;
		}
		return false;
	};

	using TSelection = SMediaPermutationsSelector<FMediaIOOutputConfiguration, FMediaIOPermutationsSelectorBuilder>;
	TSelection::FArguments Arguments;
	Arguments
		.PermutationsSource(MoveTemp(MediaConfigurations))
		.SelectedPermutation(SelectedConfiguration)
		.OnSelectionChanged(this, &FMediaIOOutputConfigurationCustomization::OnSelectionChanged)
		.OnButtonClicked(this, &FMediaIOOutputConfigurationCustomization::OnButtonClicked)
		+ TSelection::Column(FMediaIOPermutationsSelectorBuilder::NAME_OutputType)
		.Label(LOCTEXT("OutputTypeeLabel", "Output Type"))
		+ TSelection::Column(FMediaIOPermutationsSelectorBuilder::NAME_DeviceIdentifier)
		.Label(LOCTEXT("DeviceLabel", "Device"));

	if (DeviceProviderPtr->ShowOutputTransportInSelector())
	{
		Arguments
			+ TSelection::Column(FMediaIOPermutationsSelectorBuilder::NAME_TransportType)
			.Label(LOCTEXT("DestinationTypeLabel", "Destination"))
			+ TSelection::Column(FMediaIOPermutationsSelectorBuilder::NAME_QuadType)
			.Label(LOCTEXT("QuadTypeLabel", "Quad"))
			.IsColumnVisible_Lambda(QuadTypeVisible);
	}

	Arguments
		+ TSelection::Column(FMediaIOPermutationsSelectorBuilder::NAME_Resolution)
		.Label(LOCTEXT("ResolutionLabel", "Resolution"))
		+ TSelection::Column(FMediaIOPermutationsSelectorBuilder::NAME_Standard)
		.Label(LOCTEXT("StandardLabel", "Standard"))
		+ TSelection::Column(FMediaIOPermutationsSelectorBuilder::NAME_FrameRate)
		.Label(LOCTEXT("FrameRateLabel", "Frame Rate"));

	if (DeviceProviderPtr->ShowOutputKeyInSelector())
	{
		Arguments
			+ TSelection::Column(FMediaIOPermutationsSelectorBuilder::NAME_KeyPortSource)
			.Label(LOCTEXT("KeyDestinationTypeLabel", "Key Destination"))
			.IsColumnVisible_Lambda(KeyVisible);
	}

	if (DeviceProviderPtr->ShowReferenceInSelector())
	{
		Arguments
			+ TSelection::Column(FMediaIOPermutationsSelectorBuilder::NAME_OutputReference)
			.Label(LOCTEXT("ReferenceLabel", "Reference"))
			+ TSelection::Column(FMediaIOPermutationsSelectorBuilder::NAME_SyncPortSource)
			.Label(LOCTEXT("SyncSourceTypeLabel", "Sync Source"))
			.IsColumnVisible_Lambda(SyncVisible);
	}

	TSharedRef<TSelection> Selector = SNew(TSelection) = Arguments;
	PermutationSelector = Selector;
	SelectedConfiguration = Selector->GetSelectedItem();

	return Selector;
}

void FMediaIOOutputConfigurationCustomization::OnSelectionChanged(FMediaIOOutputConfiguration SelectedItem)
{
	SelectedConfiguration = SelectedItem;
}

FReply FMediaIOOutputConfigurationCustomization::OnButtonClicked() const
{
	AssignValue(SelectedConfiguration);

	TSharedPtr<SWidget> SharedPermutationSelector = PermutationSelector.Pin();
	if (SharedPermutationSelector.IsValid())
	{
		TSharedRef<SWindow> ParentContextMenuWindow = FSlateApplication::Get().FindWidgetWindow(SharedPermutationSelector.ToSharedRef()).ToSharedRef();
		FSlateApplication::Get().RequestDestroyWindow(ParentContextMenuWindow);
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
