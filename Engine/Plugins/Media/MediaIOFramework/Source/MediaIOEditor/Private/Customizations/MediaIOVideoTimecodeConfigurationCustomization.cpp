// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/MediaIOVideoTimecodeConfigurationCustomization.h"

#include "DetailWidgetRow.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Engine/TimecodeProvider.h"
#include "Framework/Application/SlateApplication.h"
#include "MediaIOPermutationsSelectorBuilder.h"
#include "IMediaIOCoreDeviceProvider.h"
#include "IMediaIOCoreModule.h"
#include "ObjectEditorUtils.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SMediaPermutationsSelector.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWindow.h"

#define LOCTEXT_NAMESPACE "MediaIOVideoTimecodeConfigurationCustomization"

namespace MediaIOVideoTimecodeConfigurationCustomization
{
	static const FName NAME_TimecodeFormat("TimecodeFormat");

	struct FMediaTimecodePermutationsSelectorBuilder
	{
		static bool IdenticalProperty(FName ColumnName, const FMediaIOVideoTimecodeConfiguration& Left, const FMediaIOVideoTimecodeConfiguration& Right)
		{
			if (ColumnName == NAME_TimecodeFormat) return Left.TimecodeFormat == Right.TimecodeFormat;
			return FMediaIOPermutationsSelectorBuilder::IdenticalProperty(ColumnName, Left.MediaConfiguration, Right.MediaConfiguration);
		}

		static bool Less(FName ColumnName, const FMediaIOVideoTimecodeConfiguration& Left, const FMediaIOVideoTimecodeConfiguration& Right)
		{
			if (ColumnName == NAME_TimecodeFormat) return (int32)Left.TimecodeFormat < (int32)Right.TimecodeFormat;
			return FMediaIOPermutationsSelectorBuilder::Less(ColumnName, Left.MediaConfiguration, Right.MediaConfiguration);
		}

		static FText GetLabel(FName ColumnName, const FMediaIOVideoTimecodeConfiguration& Item)
		{
			if (ColumnName == NAME_TimecodeFormat)
			{
				switch(Item.TimecodeFormat)
				{
				case EMediaIOAutoDetectableTimecodeFormat::LTC:
					return LOCTEXT("LtcLabel", "LTC");
				case EMediaIOAutoDetectableTimecodeFormat::VITC:
					return LOCTEXT("VITCLabel", "VITC");
				}
				return LOCTEXT("Invalid", "<Invalid>");
			}
			return FMediaIOPermutationsSelectorBuilder::GetLabel(ColumnName, Item.MediaConfiguration);
		}

		static FText GetTooltip(FName ColumnName, const FMediaIOVideoTimecodeConfiguration& Item)
		{
			if (ColumnName == NAME_TimecodeFormat) return LOCTEXT("ReferenceFrameRateTooltip", "Timecode format to read from a video signal.");
			return FMediaIOPermutationsSelectorBuilder::GetTooltip(ColumnName, Item.MediaConfiguration);
		}
	};
}

TAttribute<FText> FMediaIOVideoTimecodeConfigurationCustomization::GetContentText()
{
	FMediaIOVideoTimecodeConfiguration* Value = GetPropertyValueFromPropertyHandle<FMediaIOVideoTimecodeConfiguration>();
	return MakeAttributeLambda([this, Value]
	{
		const bool bIsAutoDetected = IsAutoDetected();
		return Value->ToText(bIsAutoDetected);
	});
}

TSharedRef<SWidget> FMediaIOVideoTimecodeConfigurationCustomization::HandleSourceComboButtonMenuContent()
{
	PermutationSelector.Reset();

	SelectedConfiguration = *GetPropertyValueFromPropertyHandle<FMediaIOVideoTimecodeConfiguration>();
	if (!SelectedConfiguration.IsValid())
	{
		TArray<FMediaIOVideoTimecodeConfiguration> MediaConfigurations;

		if (IMediaIOCoreDeviceProvider* DeviceProvider = IMediaIOCoreModule::Get().GetDeviceProvider(DeviceProviderName))
		{
			SelectedConfiguration = DeviceProvider->GetDefaultTimecodeConfiguration();
		}
	}

	bAutoDetectFormat = IsAutoDetected();

	TArray<FMediaIOVideoTimecodeConfiguration> MediaConfigurations;
	if (IMediaIOCoreDeviceProvider* DeviceProvider = IMediaIOCoreModule::Get().GetDeviceProvider(DeviceProviderName))
	{
		MediaConfigurations = DeviceProvider->GetTimecodeConfigurations();
	}

	if (MediaConfigurations.Num() == 0)
	{
		return SNew(STextBlock)
			.Text(LOCTEXT("NoConfigurationFound", "No configuration found"));
	}

	auto QuadTypeVisible = [](FName ColumnName, const TArray<FMediaIOVideoTimecodeConfiguration>& UniquePermutationsForThisColumn)
	{
		if (UniquePermutationsForThisColumn.Num() > 0)
		{
			return UniquePermutationsForThisColumn[0].MediaConfiguration.MediaConnection.TransportType == EMediaIOTransportType::QuadLink;
		}
		return false;
	};

	TSharedRef<SWidget> AutoCheckbox = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(4.f)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("AutoLabel", "Auto"))
		]
	+ SHorizontalBox::Slot()
		.Padding(4.f)
		.AutoWidth()
		[
			SNew(SCheckBox)
			.ToolTipText(LOCTEXT("AutoTooltip", "Automatically detect the video format of the timecode signal coming through this input. When disabled, the format will be enforced and the source will error out if the format differs from the expected one."))
			.IsChecked(this, &FMediaIOVideoTimecodeConfigurationCustomization::GetAutoCheckboxState)
			.OnCheckStateChanged(this, &FMediaIOVideoTimecodeConfigurationCustomization::SetAutoCheckboxState)
		];

	auto GetExtensions = [AutoCheckbox](TArray<TSharedRef<SWidget>>& OutWidgets)
	{
		OutWidgets.Add(AutoCheckbox);
	};

	using TSelection = SMediaPermutationsSelector<FMediaIOVideoTimecodeConfiguration, MediaIOVideoTimecodeConfigurationCustomization::FMediaTimecodePermutationsSelectorBuilder>;
	TSelection::FArguments Arguments;
	Arguments
		.PermutationsSource(MoveTemp(MediaConfigurations))
		.SelectedPermutation(SelectedConfiguration)
		.OnGetExtensions_Lambda(GetExtensions)
		.OnSelectionChanged(this, &FMediaIOVideoTimecodeConfigurationCustomization::OnSelectionChanged)
		.OnButtonClicked(this, &FMediaIOVideoTimecodeConfigurationCustomization::OnButtonClicked)
		+ TSelection::Column(FMediaIOPermutationsSelectorBuilder::NAME_DeviceIdentifier)
		.Label(LOCTEXT("DeviceLabel", "Device"))
		+ TSelection::Column(FMediaIOPermutationsSelectorBuilder::NAME_TransportType)
		.Label(LOCTEXT("SourceTypeLabel", "Source"))
		+ TSelection::Column(FMediaIOPermutationsSelectorBuilder::NAME_QuadType)
		.Label(LOCTEXT("QuadTypeLabel", "Quad"))
		.IsColumnVisible_Lambda(QuadTypeVisible)
		+ TSelection::Column(FMediaIOPermutationsSelectorBuilder::NAME_Resolution)
		.Label(LOCTEXT("ResolutionLabel", "Resolution"))
		.IsColumnVisible_Raw(this, &FMediaIOVideoTimecodeConfigurationCustomization::ShowAdvancedColumns)
		+ TSelection::Column(FMediaIOPermutationsSelectorBuilder::NAME_Standard)
		.Label(LOCTEXT("StandardLabel", "Standard"))
		.IsColumnVisible_Raw(this, &FMediaIOVideoTimecodeConfigurationCustomization::ShowAdvancedColumns)
		+ TSelection::Column(FMediaIOPermutationsSelectorBuilder::NAME_FrameRate)
		.Label(LOCTEXT("FrameRateLabel", "Frame Rate"))
		.IsColumnVisible_Raw(this, &FMediaIOVideoTimecodeConfigurationCustomization::ShowAdvancedColumns)
		+ TSelection::Column(MediaIOVideoTimecodeConfigurationCustomization::NAME_TimecodeFormat)
		.Label(LOCTEXT("TimecodeFormatLabel", "Format"))
		.IsColumnVisible_Raw(this, &FMediaIOVideoTimecodeConfigurationCustomization::ShowAdvancedColumns);

	TSharedRef<TSelection> Selector = SNew(TSelection) = Arguments;
	PermutationSelector = Selector;
	SelectedConfiguration = Selector->GetSelectedItem();

	return Selector;
}

void FMediaIOVideoTimecodeConfigurationCustomization::OnSelectionChanged(FMediaIOVideoTimecodeConfiguration SelectedItem)
{
	SelectedConfiguration = SelectedItem;

	if (IsAutoDetected())
	{
		SelectedConfiguration.TimecodeFormat = EMediaIOAutoDetectableTimecodeFormat::Auto;
	}
	else
	{
		if (SelectedConfiguration.TimecodeFormat == EMediaIOAutoDetectableTimecodeFormat::Auto)
		{
			SelectedConfiguration.TimecodeFormat = EMediaIOAutoDetectableTimecodeFormat::LTC;
		}
	}
}

FReply FMediaIOVideoTimecodeConfigurationCustomization::OnButtonClicked()
{
	AssignValue(SelectedConfiguration);
	// Make sure to overwrite what was in the config since the auto value is determined by the timecode provider and not the generated configs.
	SetIsAutoDetected(bAutoDetectFormat);

	TSharedPtr<SWidget> SharedPermutationSelector = PermutationSelector.Pin();
	if (SharedPermutationSelector.IsValid())
	{
		TSharedRef<SWindow> ParentContextMenuWindow = FSlateApplication::Get().FindWidgetWindow(SharedPermutationSelector.ToSharedRef()).ToSharedRef();
		FSlateApplication::Get().RequestDestroyWindow(ParentContextMenuWindow);
	}

	return FReply::Handled();
}

ECheckBoxState FMediaIOVideoTimecodeConfigurationCustomization::GetAutoCheckboxState() const
{
	return bAutoDetectFormat ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FMediaIOVideoTimecodeConfigurationCustomization::SetAutoCheckboxState(ECheckBoxState CheckboxState)
{
	bAutoDetectFormat = CheckboxState == ECheckBoxState::Checked;
}

bool FMediaIOVideoTimecodeConfigurationCustomization::ShowAdvancedColumns(FName ColumnName, const TArray<FMediaIOVideoTimecodeConfiguration>& UniquePermutationsForThisColumn) const
{
	return !bAutoDetectFormat;
}

bool FMediaIOVideoTimecodeConfigurationCustomization::IsAutoDetected() const
{
	bool bAutoDetectTimecode = true;

	for (UObject* CustomizedObject : GetCustomizedObjects())
	{
		if (UTimecodeProvider* Provider = Cast<UTimecodeProvider>(CustomizedObject))
		{
			if (!Provider->IsAutoDetected())
			{
				bAutoDetectTimecode = false;
				break;
			}
		}
	}

	return bAutoDetectTimecode;
}

void FMediaIOVideoTimecodeConfigurationCustomization::SetIsAutoDetected(bool Value)
{
	FScopedTransaction Transaction{ LOCTEXT("MediaIOAutoDetectTimecode", "Auto Detect Timecode") };

	for (UObject* CustomizedObject : GetCustomizedObjects())
	{
		if (UTimecodeProvider* Provider = Cast<UTimecodeProvider>(CustomizedObject))
		{
			Provider->Modify();
			Provider->SetIsAutoDetected(Value);
		}
	}
}

#undef LOCTEXT_NAMESPACE
