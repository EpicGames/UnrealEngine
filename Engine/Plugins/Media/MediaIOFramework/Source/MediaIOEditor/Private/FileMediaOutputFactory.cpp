// Copyright Epic Games, Inc. All Rights Reserved.

#include "FileMediaOutputFactory.h"
#include "FileMediaOutput.h"
#include "AssetTypeCategories.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FileMediaOutputFactory)


/* UFileMediaOutputFactory structors
 *****************************************************************************/

UFileMediaOutputFactory::UFileMediaOutputFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UFileMediaOutput::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}


/* UFactory overrides
 *****************************************************************************/

UObject* UFileMediaOutputFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UFileMediaOutput>(InParent, InClass, InName, Flags);
}
