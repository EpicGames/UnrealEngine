// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "FileMediaOutputFactory.generated.h"

#define UE_API MEDIAIOEDITOR_API

/**
 * Implements a factory for UFileMediaOutput objects.
 */
UCLASS(MinimalAPI)
class UFileMediaOutputFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

public:
	UE_API virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
};

#undef UE_API
