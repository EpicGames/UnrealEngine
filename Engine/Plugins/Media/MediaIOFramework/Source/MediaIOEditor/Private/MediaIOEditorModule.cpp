// Copyright Epic Games, Inc. All Rights Reserved.


#include "MediaIOEditorModule.h"

#include "Brushes/SlateImageBrush.h"
#include "MediaCapture.h"
#include "Misc/Paths.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"

#include "Customizations/MediaCaptureOptionsCustomization.h"
#include "Customizations/MediaIODeviceCustomization.h"
#include "Customizations/MediaIOConfigurationCustomization.h"
#include "Customizations/MediaIOInputConfigurationCustomization.h"
#include "Customizations/MediaIOOutputConfigurationCustomization.h"
#include "Customizations/MediaIOVideoTimecodeConfigurationCustomization.h"


DEFINE_LOG_CATEGORY(LogMediaIOEditor);

class FMediaIOEditorModule : public IModuleInterface
{
	TUniquePtr<FSlateStyleSet> StyleInstance;

	virtual void StartupModule() override
	{
		RegisterCustomizations();
		RegisterStyle();
	}

	virtual void ShutdownModule() override
	{
		if (!UObjectInitialized() && !IsEngineExitRequested())
		{
			UnregisterStyle();
			UnregisterCustomizations();
		}
	}

private:
	/** Register details view customizations. */
	void RegisterCustomizations()
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomPropertyTypeLayout(FMediaCaptureOptions::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMediaCaptureOptionsCustomization::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout(FMediaIOConfiguration::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMediaIOConfigurationCustomization::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout(FMediaIODevice::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMediaIODeviceCustomization::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout(FMediaIOInputConfiguration::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMediaIOInputConfigurationCustomization::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout(FMediaIOOutputConfiguration::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMediaIOOutputConfigurationCustomization::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout(FMediaIOVideoTimecodeConfiguration::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMediaIOVideoTimecodeConfigurationCustomization::MakeInstance));
	}

	/** Unregister details view customizations. */
	void UnregisterCustomizations()
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomPropertyTypeLayout(FMediaIOVideoTimecodeConfiguration::StaticStruct()->GetFName());
		PropertyModule.UnregisterCustomPropertyTypeLayout(FMediaCaptureOptions::StaticStruct()->GetFName());
		PropertyModule.UnregisterCustomPropertyTypeLayout(FMediaIOOutputConfiguration::StaticStruct()->GetFName());
		PropertyModule.UnregisterCustomPropertyTypeLayout(FMediaIOInputConfiguration::StaticStruct()->GetFName());
		PropertyModule.UnregisterCustomPropertyTypeLayout(FMediaIODevice::StaticStruct()->GetFName());
		PropertyModule.UnregisterCustomPropertyTypeLayout(FMediaIOConfiguration::StaticStruct()->GetFName());
	}

	/** Register slate style */
	void RegisterStyle()
	{
#define IMAGE_BRUSH(RelativePath, ...) FSlateImageBrush(StyleInstance->RootToContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)

		StyleInstance = MakeUnique<FSlateStyleSet>(TEXT("MediaIOStyle"));
		StyleInstance->SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Media/MediaIOFramework/Content/Editor/Icons/"));

		StyleInstance->Set("ClassThumbnail.FileMediaOutput", new IMAGE_BRUSH("FileMediaOutput_64x", FVector2D(64, 64)));
		StyleInstance->Set("ClassIcon.FileMediaOutput", new IMAGE_BRUSH("FileMediaOutput_16x", FVector2D(16, 16)));

#undef IMAGE_BRUSH

		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance.Get());
	}

	/** Unregister slate style */
	void UnregisterStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance.Get());
	}
};

IMPLEMENT_MODULE(FMediaIOEditorModule, MediaIOEditor)
