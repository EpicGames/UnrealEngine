// Copyright Epic Games, Inc. All Rights Reserved.

#include "MediaIOPermutationsSelectorBuilder.h"

#include "CommonFrameRates.h"
#include "MediaIOCoreCommonDisplayMode.h"

#define LOCTEXT_NAMESPACE "MediaIOPermutationsSelectorBuilder"

namespace MediaIOPermutationsSelectorBuilder
{
	FText LinkTypeToPrettyText(EMediaIOTransportType InLinkType, int32 InChannel)
	{
		switch (InLinkType)
		{
		case EMediaIOTransportType::SingleLink: return FText::Format(LOCTEXT("SingleLinkLabel", "Single Link {0}"), FText::AsNumber(InChannel));
		case EMediaIOTransportType::DualLink: return FText::Format(LOCTEXT("DualLinkLabel", "Dual Link {0}-{1}"), FText::AsNumber(InChannel), FText::AsNumber(InChannel + 1));
		case EMediaIOTransportType::QuadLink: return FText::Format(LOCTEXT("QuadLinkLabel", "Quad Link {0}-{1}"), FText::AsNumber(InChannel), FText::AsNumber(InChannel + 3));
		case EMediaIOTransportType::HDMI: return FText::Format(LOCTEXT("HDMILinkLabel", "HDMI {0}"), FText::AsNumber(InChannel));
		}
		return FText::GetEmpty();
	}

	FText QuadLinkTypeToPrettyText(EMediaIOQuadLinkTransportType InLinkType)
	{
		switch (InLinkType)
		{
		case EMediaIOQuadLinkTransportType::SquareDivision: return LOCTEXT("QuadLinkSquareLabel", "Square Division");
		case EMediaIOQuadLinkTransportType::TwoSampleInterleave: return LOCTEXT("QuadLinkSILabel", "Sample Interleave");
		}
		return FText::GetEmpty();
	}

	FText StandardTypeToPrettyText(EMediaIOStandardType InStandard)
	{
		switch (InStandard)
		{
		case EMediaIOStandardType::Progressive: return LOCTEXT("ProgressiveLabel", "Progressive");
		case EMediaIOStandardType::Interlaced: return LOCTEXT("InterlacedLabel", "Interlaced");
		case EMediaIOStandardType::ProgressiveSegmentedFrame: return LOCTEXT("ProgressiveSegmentedFrameLabel", "PSF");
		}
		return FText::GetEmpty();
	}
}

/**
 * FNames
 */
const FName FMediaIOPermutationsSelectorBuilder::NAME_DeviceIdentifier = "DeviceIdentifier";
const FName FMediaIOPermutationsSelectorBuilder::NAME_TransportType = "SourceType";
const FName FMediaIOPermutationsSelectorBuilder::NAME_QuadType = "QuadType";
const FName FMediaIOPermutationsSelectorBuilder::NAME_Resolution = "Resolution";
const FName FMediaIOPermutationsSelectorBuilder::NAME_Standard = "Standard";
const FName FMediaIOPermutationsSelectorBuilder::NAME_FrameRate = "FrameRate";

const FName FMediaIOPermutationsSelectorBuilder::NAME_InputType = "InputType";
const FName FMediaIOPermutationsSelectorBuilder::NAME_OutputType = "OutputType";
const FName FMediaIOPermutationsSelectorBuilder::NAME_KeyPortSource = "KeyPortSource";
const FName FMediaIOPermutationsSelectorBuilder::NAME_OutputReference = "OutputReference";
const FName FMediaIOPermutationsSelectorBuilder::NAME_SyncPortSource = "SyncPortSource";

/**
 * With FMediaIOConnection
 */
bool FMediaIOPermutationsSelectorBuilder::IdenticalProperty(FName ColumnName, const FMediaIOConnection& Left, const FMediaIOConnection& Right)
{
	if (ColumnName == NAME_DeviceIdentifier) return Left.Device.DeviceIdentifier == Right.Device.DeviceIdentifier;
	if (ColumnName == NAME_TransportType) return Left.TransportType == Right.TransportType && Left.PortIdentifier == Right.PortIdentifier;
	if (ColumnName == NAME_QuadType) return Left.TransportType == EMediaIOTransportType::QuadLink ? Left.QuadTransportType == Right.QuadTransportType : true;

	return false;
}


bool FMediaIOPermutationsSelectorBuilder::Less(FName ColumnName, const FMediaIOConnection& Left, const FMediaIOConnection& Right)
{
	if (ColumnName == NAME_DeviceIdentifier) return Left.Device.DeviceIdentifier < Right.Device.DeviceIdentifier;
	if (ColumnName == NAME_TransportType)
	{
		if (Left.TransportType == Right.TransportType)
		{
			return Left.PortIdentifier < Right.PortIdentifier;
		}
		return (int32)Left.TransportType < (int32)Right.TransportType;
	}
	if (ColumnName == NAME_QuadType) return (Left.TransportType == EMediaIOTransportType::QuadLink) ? (int32)Left.QuadTransportType < (int32)Right.QuadTransportType : false;

	return false;
}


FText FMediaIOPermutationsSelectorBuilder::GetLabel(FName ColumnName, const FMediaIOConnection& Item)
{
	if (ColumnName == NAME_DeviceIdentifier) return FText::FromName(Item.Device.DeviceName);
	if (ColumnName == NAME_TransportType) return MediaIOPermutationsSelectorBuilder::LinkTypeToPrettyText(Item.TransportType, Item.PortIdentifier);
	if (ColumnName == NAME_QuadType) return MediaIOPermutationsSelectorBuilder::QuadLinkTypeToPrettyText(Item.QuadTransportType);

	return FText::GetEmpty();
}


FText FMediaIOPermutationsSelectorBuilder::GetTooltip(FName ColumnName, const FMediaIOConnection& Item)
{
	if (ColumnName == NAME_DeviceIdentifier) return FText::FromString(FString::Printf(TEXT("%s as identifier: %d"), *Item.Device.DeviceName.ToString(), Item.Device.DeviceIdentifier));
	if (ColumnName == NAME_TransportType) return FText::GetEmpty();
	if (ColumnName == NAME_QuadType) return LOCTEXT("QuadTypeTooltip", "Output can be Square Division Quad Split (SQ) or Two-Sample Interleave (SI).");

	return FText::GetEmpty();
}


/**
 * With FMediaIOMode
 */
bool FMediaIOPermutationsSelectorBuilder::IdenticalProperty(FName ColumnName, const FMediaIOMode& Left, const FMediaIOMode& Right)
{
	if (ColumnName == NAME_Resolution) return Left.Resolution == Right.Resolution;
	if (ColumnName == NAME_Standard) return Left.Standard == Right.Standard;
	if (ColumnName == NAME_FrameRate) return Left.FrameRate == Right.FrameRate;

	return false;
}


bool FMediaIOPermutationsSelectorBuilder::Less(FName ColumnName, const FMediaIOMode& Left, const FMediaIOMode& Right)
{
	if (ColumnName == NAME_Resolution) return Left.Resolution.SizeSquared() < Right.Resolution.SizeSquared();
	if (ColumnName == NAME_Standard) return (int32)Left.Standard < (int32)Right.Standard;
	if (ColumnName == NAME_FrameRate) return Left.FrameRate.AsDecimal() < Right.FrameRate.AsDecimal();

	return false;
}


FText FMediaIOPermutationsSelectorBuilder::GetLabel(FName ColumnName, const FMediaIOMode& Item)
{
	if (ColumnName == NAME_Resolution) return FMediaIOCommonDisplayModes::GetMediaIOCommonDisplayModeResolutionInfoName(Item.Resolution.X, Item.Resolution.Y);
	if (ColumnName == NAME_Standard) return MediaIOPermutationsSelectorBuilder::StandardTypeToPrettyText(Item.Standard);
	if (ColumnName == NAME_FrameRate) return Item.FrameRate.ToPrettyText();

	return FText::GetEmpty();
}


FText FMediaIOPermutationsSelectorBuilder::GetTooltip(FName ColumnName, const FMediaIOMode& Item)
{
	if (ColumnName == NAME_Resolution) return FText::FromString(FString::Printf(TEXT("%dx%d"), Item.Resolution.X, Item.Resolution.Y));
	if (ColumnName == NAME_Standard) return FText::GetEmpty();
	if (ColumnName == NAME_FrameRate)
	{
		if (const FCommonFrameRateInfo* Found = FCommonFrameRates::Find(Item.FrameRate))
		{
			return Found->Description;
		}
		return Item.FrameRate.ToPrettyText();
	}

	return FText::GetEmpty();
}


/**
 * With FMediaIOMode
 */
bool FMediaIOPermutationsSelectorBuilder::IdenticalProperty(FName ColumnName, const FMediaIOConfiguration& Left, const FMediaIOConfiguration& Right)
{
	if (ColumnName == NAME_Resolution || ColumnName == NAME_Standard || ColumnName == NAME_FrameRate)
	{
		return IdenticalProperty(ColumnName, Left.MediaMode, Right.MediaMode);
	}
	return IdenticalProperty(ColumnName, Left.MediaConnection, Right.MediaConnection);
}


bool FMediaIOPermutationsSelectorBuilder::Less(FName ColumnName, const FMediaIOConfiguration& Left, const FMediaIOConfiguration& Right)
{
	if (ColumnName == NAME_Resolution || ColumnName == NAME_Standard || ColumnName == NAME_FrameRate)
	{
		return Less(ColumnName, Left.MediaMode, Right.MediaMode);
	}
	return Less(ColumnName, Left.MediaConnection, Right.MediaConnection);
}

FText FMediaIOPermutationsSelectorBuilder::GetLabel(FName ColumnName, const FMediaIOConfiguration& Item)
{
	if (ColumnName == NAME_Resolution || ColumnName == NAME_Standard || ColumnName == NAME_FrameRate)
	{
		return GetLabel(ColumnName, Item.MediaMode);
	}
	return GetLabel(ColumnName, Item.MediaConnection);
}


FText FMediaIOPermutationsSelectorBuilder::GetTooltip(FName ColumnName, const FMediaIOConfiguration& Item)
{
	if (ColumnName == NAME_Resolution || ColumnName == NAME_Standard || ColumnName == NAME_FrameRate)
	{
		return GetTooltip(ColumnName, Item.MediaMode);
	}
	return GetTooltip(ColumnName, Item.MediaConnection);
}


bool FMediaIOPermutationsSelectorBuilder::IdenticalProperty(FName ColumnName, const FMediaIOInputConfiguration& Left, const FMediaIOInputConfiguration& Right)
{
	if (ColumnName == NAME_InputType) return Left.InputType == Right.InputType;
	if (ColumnName == NAME_KeyPortSource) return Left.InputType == EMediaIOInputType::FillAndKey ? Left.KeyPortIdentifier == Right.KeyPortIdentifier : true;

	return IdenticalProperty(ColumnName, Left.MediaConfiguration, Right.MediaConfiguration);
}


bool FMediaIOPermutationsSelectorBuilder::Less(FName ColumnName, const FMediaIOInputConfiguration& Left, const FMediaIOInputConfiguration& Right)
{
	if (ColumnName == NAME_InputType) return (int32)Left.InputType < (int32)Right.InputType;
	if (ColumnName == NAME_KeyPortSource) return Left.KeyPortIdentifier < Right.KeyPortIdentifier;

	return Less(ColumnName, Left.MediaConfiguration, Right.MediaConfiguration);
}


FText FMediaIOPermutationsSelectorBuilder::GetLabel(FName ColumnName, const FMediaIOInputConfiguration& Item)
{
	if (ColumnName == NAME_InputType) return Item.InputType == EMediaIOInputType::Fill ? LOCTEXT("Fill", "Fill") : LOCTEXT("FillAndKey", "Fill and Key");
	if (ColumnName == NAME_KeyPortSource) return MediaIOPermutationsSelectorBuilder::LinkTypeToPrettyText(Item.MediaConfiguration.MediaConnection.TransportType, Item.KeyPortIdentifier);

	return GetLabel(ColumnName, Item.MediaConfiguration);
}


FText FMediaIOPermutationsSelectorBuilder::GetTooltip(FName ColumnName, const FMediaIOInputConfiguration& Item)
{
	if (ColumnName == NAME_InputType) return LOCTEXT("InputTypeTooltip", "Whether to input the fill or the fill and key.");
	if (ColumnName == NAME_KeyPortSource) return FText::GetEmpty();

	return GetTooltip(ColumnName, Item.MediaConfiguration);
}


bool FMediaIOPermutationsSelectorBuilder::IdenticalProperty(FName ColumnName, const FMediaIOOutputConfiguration& Left, const FMediaIOOutputConfiguration& Right)
{
	if (ColumnName == NAME_OutputType) return Left.OutputType == Right.OutputType;
	if (ColumnName == NAME_KeyPortSource) return Left.OutputType == EMediaIOOutputType::FillAndKey ? Left.KeyPortIdentifier == Right.KeyPortIdentifier : true;
	if (ColumnName == NAME_OutputReference) return Left.OutputReference == Right.OutputReference;
	if (ColumnName == NAME_SyncPortSource) return Left.OutputReference == EMediaIOReferenceType::Input ? Left.ReferencePortIdentifier == Right.ReferencePortIdentifier : true;

	return IdenticalProperty(ColumnName, Left.MediaConfiguration, Right.MediaConfiguration);
}


bool FMediaIOPermutationsSelectorBuilder::Less(FName ColumnName, const FMediaIOOutputConfiguration& Left, const FMediaIOOutputConfiguration& Right)
{
	if (ColumnName == NAME_OutputType) return (int32)Left.OutputType < (int32)Right.OutputType;
	if (ColumnName == NAME_KeyPortSource) return Left.KeyPortIdentifier < Right.KeyPortIdentifier;
	if (ColumnName == NAME_OutputReference) return (int32)Left.OutputReference < (int32)Right.OutputReference;
	if (ColumnName == NAME_SyncPortSource) return Left.ReferencePortIdentifier < Right.ReferencePortIdentifier;

	return Less(ColumnName, Left.MediaConfiguration, Right.MediaConfiguration);
}


FText FMediaIOPermutationsSelectorBuilder::GetLabel(FName ColumnName, const FMediaIOOutputConfiguration& Item)
{
	if (ColumnName == NAME_OutputType) return Item.OutputType == EMediaIOOutputType::Fill ? LOCTEXT("Fill", "Fill") : LOCTEXT("FillAndKey", "Fill and Key");
	if (ColumnName == NAME_KeyPortSource) return MediaIOPermutationsSelectorBuilder::LinkTypeToPrettyText(Item.MediaConfiguration.MediaConnection.TransportType, Item.KeyPortIdentifier);
	if (ColumnName == NAME_OutputReference)
	{
		switch (Item.OutputReference)
		{
		case EMediaIOReferenceType::FreeRun: return LOCTEXT("FreeRun", "Free Run");
		case EMediaIOReferenceType::External: return LOCTEXT("External", "External");
		case EMediaIOReferenceType::Input: return LOCTEXT("Input", "Input");
		}
	}
	if (ColumnName == NAME_SyncPortSource) return MediaIOPermutationsSelectorBuilder::LinkTypeToPrettyText(Item.MediaConfiguration.MediaConnection.TransportType, Item.ReferencePortIdentifier);

	return GetLabel(ColumnName, Item.MediaConfiguration);
}


FText FMediaIOPermutationsSelectorBuilder::GetTooltip(FName ColumnName, const FMediaIOOutputConfiguration& Item)
{
	if (ColumnName == NAME_OutputType) return LOCTEXT("OutputTypeTooltip", "Whether to output the fill or the fill and key.");
	if (ColumnName == NAME_KeyPortSource) return FText::GetEmpty();
	if (ColumnName == NAME_OutputReference) return LOCTEXT("OutputReferenceTooltip", "The Device output is synchronized with either its internal clock, an external reference, or an other input.");
	if (ColumnName == NAME_SyncPortSource) return FText::GetEmpty();

	return GetTooltip(ColumnName, Item.MediaConfiguration);
}

#undef LOCTEXT_NAMESPACE
