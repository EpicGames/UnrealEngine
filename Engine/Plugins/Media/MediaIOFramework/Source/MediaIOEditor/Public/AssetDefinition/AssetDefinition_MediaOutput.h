// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AssetDefinitionDefault.h"
#include "AssetDefinition_MediaOutput.generated.h"

#define UE_API MEDIAIOEDITOR_API

UCLASS(MinimalAPI, Abstract)
class UAssetDefinition_MediaOutput : public UAssetDefinitionDefault
{
	GENERATED_BODY()
public:
	// UAssetDefinition Begin
	UE_API virtual FLinearColor GetAssetColor() const override;
	UE_API virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	UE_API virtual bool CanImport() const override;
	// UAssetDefinition End
};

#undef UE_API
