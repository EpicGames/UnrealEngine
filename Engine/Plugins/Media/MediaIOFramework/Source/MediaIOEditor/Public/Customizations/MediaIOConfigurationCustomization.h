// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Customizations/MediaIOCustomizationBase.h"
#include "Input/Reply.h"
#include "MediaIOCoreDefinitions.h"

#define UE_API MEDIAIOEDITOR_API

/**
 * Implements a details view customization for the FMediaIOConfiguration
 */
class FMediaIOConfigurationCustomization : public FMediaIOCustomizationBase
{
public:
	static UE_API TSharedRef<IPropertyTypeCustomization> MakeInstance();

protected:
	//~ FMediaIOCustomizationBase interface
	virtual void OnCustomizeObjects(const TArray<UObject*>& InCustomizedObjects) override;

private:
	UE_API virtual TAttribute<FText> GetContentText() override;
	UE_API virtual TSharedRef<SWidget> HandleSourceComboButtonMenuContent() override;

	UE_API ECheckBoxState GetAutoCheckboxState() const;
	UE_API void SetAutoCheckboxState(ECheckBoxState CheckboxState);
	UE_API void OnSelectionChanged(FMediaIOConfiguration SelectedItem);
	UE_API FReply OnButtonClicked();
	UE_API bool ShowAdvancedColumns(FName ColumnName, const TArray<FMediaIOConfiguration>& UniquePermutationsForThisColumn) const;
	UE_API bool IsAutoDetected() const;
	UE_API void SetIsAutoDetected(bool Value);

private:
	TWeakPtr<SWidget> PermutationSelector;
	FMediaIOConfiguration SelectedConfiguration;
	bool bAutoDetectFormat = false;
	/** List of customized objects. */
	TArray<TWeakObjectPtr<UObject>> Objects;
};

#undef UE_API
