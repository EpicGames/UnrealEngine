// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "Misc/Attribute.h"
#include "PropertyHandle.h"
#include "Widgets/SWidget.h"

#define UE_API MEDIAIOEDITOR_API

/**
 * Base implementation of different MediaIO details view customization
 */
class FMediaIOCustomizationBase : public IPropertyTypeCustomization
{
public:
	/** IPropertyTypeCustomization interface */
	UE_API virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> InPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& PropertyTypeCustomizationUtils) override;
	UE_API virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> InPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& PropertyTypeCustomizationUtils) override;

protected:
	template<class T>
	T* GetPropertyValueFromPropertyHandle()
	{
			FProperty* Property = MediaProperty->GetProperty();
			check(Property);
			check(CastField<FStructProperty>(Property));
			check(CastField<FStructProperty>(Property)->Struct);
			check(CastField<FStructProperty>(Property)->Struct->IsChildOf(T::StaticStruct()));

			TArray<void*> RawData;
			MediaProperty->AccessRawData(RawData);

			check(RawData.Num() == 1);
			T* MediaValue = reinterpret_cast<T*>(RawData[0]);
			check(MediaValue);
			return MediaValue;
	}
	
	template<class T>
	void AssignValue(const T& NewValue) const
	{
		AssignValueImpl(reinterpret_cast<const void*>(&NewValue));
	}

	virtual TAttribute<FText> GetContentText() = 0;
	virtual TSharedRef<SWidget> HandleSourceComboButtonMenuContent() = 0;
	TSharedPtr<IPropertyHandle> GetMediaProperty() const { return MediaProperty; };
	UE_API TArray<UObject*> GetCustomizedObjects() const;

	/** Used to give a chance to the child implementations to get access to the customized objects as soon as possible. */
	virtual void OnCustomizeObjects(const TArray<UObject*>& InCustomizedObjects) {}

	FName DeviceProviderName;

private:
	UE_API void AssignValueImpl(const void* NewValue) const;

	/** Pointer to the property handle. */
	TSharedPtr<IPropertyHandle> MediaProperty;
};

#undef UE_API
