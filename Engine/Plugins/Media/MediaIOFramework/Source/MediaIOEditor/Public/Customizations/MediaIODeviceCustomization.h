// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Customizations/MediaIOCustomizationBase.h"

#define UE_API MEDIAIOEDITOR_API

/**
 * Implements a details view customization for the FMediaIODevice
 */
class FMediaIODeviceCustomization : public FMediaIOCustomizationBase
{
public:
	static UE_API TSharedRef<IPropertyTypeCustomization> MakeInstance();

private:
	UE_API virtual TAttribute<FText> GetContentText() override;
	UE_API virtual TSharedRef<SWidget> HandleSourceComboButtonMenuContent() override;
};

#undef UE_API
