// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Customizations/MediaIOCustomizationBase.h"
#include "Input/Reply.h"
#include "MediaIOCoreDefinitions.h"

#define UE_API MEDIAIOEDITOR_API

/**
 * Implements a details view customization for the FMediaIOInputConfiguration
 */
class FMediaIOInputConfigurationCustomization : public FMediaIOCustomizationBase
{
public:
	static UE_API TSharedRef<IPropertyTypeCustomization> MakeInstance();

private:
	UE_API virtual TAttribute<FText> GetContentText() override;
	UE_API virtual TSharedRef<SWidget> HandleSourceComboButtonMenuContent() override;

	UE_API void OnSelectionChanged(FMediaIOInputConfiguration SelectedItem);
	UE_API FReply OnButtonClicked() const;

	TWeakPtr<SWidget> PermutationSelector;
	FMediaIOInputConfiguration SelectedConfiguration;
};

#undef UE_API
