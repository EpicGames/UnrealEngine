// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Customizations/MediaIOCustomizationBase.h"
#include "MediaIOCoreDefinitions.h"
#include "Input/Reply.h"

/**
 * Implements a details view customization for the video timecode configuration.
 */
class FMediaIOVideoTimecodeConfigurationCustomization : public FMediaIOCustomizationBase
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance() { return MakeShared<FMediaIOVideoTimecodeConfigurationCustomization>(); }
	
private:
	virtual TAttribute<FText> GetContentText() override;
	virtual TSharedRef<SWidget> HandleSourceComboButtonMenuContent() override;

	void OnSelectionChanged(FMediaIOVideoTimecodeConfiguration SelectedItem);
	FReply OnButtonClicked();
	ECheckBoxState GetAutoCheckboxState() const;
	void SetAutoCheckboxState(ECheckBoxState CheckboxState);
	bool ShowAdvancedColumns(FName ColumnName, const TArray<FMediaIOVideoTimecodeConfiguration>& UniquePermutationsForThisColumn) const;
	bool IsAutoDetected() const;
	void SetIsAutoDetected(bool Value);

private:
	TWeakPtr<SWidget> PermutationSelector;
	FMediaIOVideoTimecodeConfiguration SelectedConfiguration;
	bool bAutoDetectFormat = false;
};
