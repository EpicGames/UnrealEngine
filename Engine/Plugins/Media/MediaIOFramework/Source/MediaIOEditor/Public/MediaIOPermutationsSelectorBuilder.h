// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MediaIOCoreDefinitions.h"

#define UE_API MEDIAIOEDITOR_API


struct FMediaIOPermutationsSelectorBuilder
{
	static UE_API const FName NAME_DeviceIdentifier;
	static UE_API const FName NAME_TransportType;
	static UE_API const FName NAME_QuadType;
	static UE_API const FName NAME_Resolution;
	static UE_API const FName NAME_Standard;
	static UE_API const FName NAME_FrameRate;

	static UE_API const FName NAME_InputType;
	static UE_API const FName NAME_OutputType;
	static UE_API const FName NAME_KeyPortSource;
	static UE_API const FName NAME_OutputReference;
	static UE_API const FName NAME_SyncPortSource;

	static UE_API bool IdenticalProperty(FName ColumnName, const FMediaIOConnection& Left, const FMediaIOConnection& Right);
	static UE_API bool Less(FName ColumnName, const FMediaIOConnection& Left, const FMediaIOConnection& Right);
	static UE_API FText GetLabel(FName ColumnName, const FMediaIOConnection& Item);
	static UE_API FText GetTooltip(FName ColumnName, const FMediaIOConnection& Item);
	
	static UE_API bool IdenticalProperty(FName ColumnName, const FMediaIOMode& Left, const FMediaIOMode& Right);
	static UE_API bool Less(FName ColumnName, const FMediaIOMode& Left, const FMediaIOMode& Right);
	static UE_API FText GetLabel(FName ColumnName, const FMediaIOMode& Item);
	static UE_API FText GetTooltip(FName ColumnName, const FMediaIOMode& Item);

	static UE_API bool IdenticalProperty(FName ColumnName, const FMediaIOConfiguration& Left, const FMediaIOConfiguration& Right);
	static UE_API bool Less(FName ColumnName, const FMediaIOConfiguration& Left, const FMediaIOConfiguration& Right);
	static UE_API FText GetLabel(FName ColumnName, const FMediaIOConfiguration& Item);
	static UE_API FText GetTooltip(FName ColumnName, const FMediaIOConfiguration& Item);

	static UE_API bool IdenticalProperty(FName ColumnName, const FMediaIOInputConfiguration& Left, const FMediaIOInputConfiguration& Right);
	static UE_API bool Less(FName ColumnName, const FMediaIOInputConfiguration& Left, const FMediaIOInputConfiguration& Right);
	static UE_API FText GetLabel(FName ColumnName, const FMediaIOInputConfiguration& Item);
	static UE_API FText GetTooltip(FName ColumnName, const FMediaIOInputConfiguration& Item);

	static UE_API bool IdenticalProperty(FName ColumnName, const FMediaIOOutputConfiguration& Left, const FMediaIOOutputConfiguration& Right);
	static UE_API bool Less(FName ColumnName, const FMediaIOOutputConfiguration& Left, const FMediaIOOutputConfiguration& Right);
	static UE_API FText GetLabel(FName ColumnName, const FMediaIOOutputConfiguration& Item);
	static UE_API FText GetTooltip(FName ColumnName, const FMediaIOOutputConfiguration& Item);
};

#undef UE_API
