// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/SlateDelegates.h"
#include "Misc/Optional.h"
#include "Types/SlateStructs.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"


/**
 * A widget that let you select a single permutation from a list . It groups the values into categories and removes duplicates inside that category.
 *
 * A trivial use case appears below:
 *
 *   struct FMyItem { int32 A; int32 B; }
 *   struct FMyBuilder
 *   {
 *      static const FName ColumnA = "A";
 *      static const FName ColumnB = "B";
 *      static bool IdenticalProperty(FName ColumnName, const FMyItem& Left, const FMyItem& Right) { return ColumnName == ColumnA ? Left.A == Right.A : Left.B == Right.B; }
 *      static bool Less(FName ColumnName, const FMyItem& Left, const FMyItem& Right) { return ColumnName == ColumnA ? Left.A < Right.A : Left.B < Right.B; }
 *      static FText GetLabel(FName ColumnName, const FMyItem& Item) { return ColumnName == ColumnA ? FText::AsNumber(Item.A) : FText::AsNumber(Item.B); }
 *      static FText GetTooltip(FName ColumnName, const FMyItem& Item) { return LOCTEXT("Tooltip", "Tooltip"); }
 *   };
 *
 *   TArray< FMyItem > Items;
 *   FMyItem Value1;
 *   Value1.A = 1; Value1.B = 2;
 *   Items.Add( Value1 );
 *   FMyItem Value2;
 *   Value2.A = 1; Value2.B = 3;
 *   Items.Add( Value2 );
 *
 *   using TSelection = SMediaPermutationsSelector< FMyItem, FMyBuilder >;
 *   SNew( TSelection )
 *     .PermutationsSource( MoveTemp(Items) )
 *     .SelectedPermutation( Value2 )
 *     + TSelection::Column( FMyBuilder::A );
 *     .Label( LOCTEXT("ExampleA", "The A") )
 *     + TSelection::Column( FMyBuilder::B );
 *     .Label( LOCTEXT("ExampleB", "The B") )
 *
 * In the example, we make all 2 columns. One for A and one for B.
 * The first column will have 1 element: "1". The second column will have 2 elements: "2", "3"
 *
 */

class SHorizontalBox;

template <typename ItemType>
class TMediaPermutationsSelectorBuilder
{
public:
	static bool IdenticalProperty(FName ColumnName, ItemType Left, ItemType Right) { return Left == Right; }
	static bool Less(FName ColumnName, ItemType Left, ItemType Right) { return Left < Right; }
	static FText GetLabel(FName ColumnName, ItemType Item) { return FText::FromName(ColumnName); }
	static FText GetTooltip(FName ColumnName, ItemType Item) { return FText::FromName(ColumnName); }
};

template <typename ItemType, typename ItemBuilder = TMediaPermutationsSelectorBuilder<ItemType>>
class SMediaPermutationsSelector : public SCompoundWidget
{
public:
	using ThisClass = SMediaPermutationsSelector<ItemType, ItemBuilder>;

	/** A delegate type invoked when the selection changes. */
	DECLARE_DELEGATE_OneParam(FOnSelectionChanged, ItemType /* NewItemSelected*/);

	/**
	 * A delegate type invoked when we fill a column and wants to check if it should be visible.
	 * @param ColumnName the name of the ColumnName
	 * @param UniqueItemInColumn  List of items used to generate the column. Multiples items may be shared the value but only one will be in that list.
	 * @return true if the column should be visible
	 */
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FIsColumnVisible, FName /*ColumnName*/, const TArray<ItemType>& /*UniquePermutationsForThisColumn */);

	/**
	 * A delegate called to add additional widgets to the left of the apply button.
	 */
	DECLARE_DELEGATE_OneParam(FOnGetExtensions, TArray<TSharedRef<SWidget>>& /*OutExtensions*/);

public:
	/** Describes a single column */
	class FColumn
	{
	public:
		SLATE_BEGIN_ARGS(FColumn)
		{}
			/** A unique ID for this property, so that it can be saved and restored. */
			SLATE_ARGUMENT(FName, ColumnName)
			/** Text to use as the Column header. */
			SLATE_ATTRIBUTE(FText, Label)
			/** Text to use as the Column tooltip. */
			SLATE_ATTRIBUTE(FText, Tooltip)
			/** Delegate to invoke when build the column and check the visibility. */
			SLATE_EVENT(FIsColumnVisible, IsColumnVisible)

		SLATE_END_ARGS()

		FColumn(const FArguments& InArgs)
			: ColumnName(InArgs._ColumnName)
			, Label(InArgs._Label)
			, Tooltip(InArgs._Tooltip)
			, IsColumnVisible(InArgs._IsColumnVisible)
		{ }

	public:
		/** A unique ID for this property, so that it can be saved and restored. */
		FName ColumnName;

		/** Text to use as the Column header. */
		TAttribute< FText > Label;

		/** Text to use as the Column tooltip. */
		TAttribute< FText > Tooltip;

		/** Widget created by this menu. */
		TSharedPtr< SWidget > Widget;

		/** Delegate to invoke when build the column and check the visibility. */
		FIsColumnVisible IsColumnVisible;
	};

public:
	SLATE_BEGIN_ARGS(ThisClass)
		: _ColumnHeight(200)
	{}
		/** Array of columns */
		SLATE_SUPPORTS_SLOT_WITH_ARGS(FColumn)
		/** Array of data items that we are displaying */
		SLATE_ARGUMENT(TArray<ItemType>, PermutationsSource)
		/** Default selected item in ItemsSource */
		SLATE_ARGUMENT(TOptional<ItemType>, SelectedPermutation)
		/** Desired height of the columns */
		SLATE_ATTRIBUTE(FOptionalSize, ColumnHeight)
		/** Override the "apply" button widget */
		SLATE_ARGUMENT(TSharedPtr<SWidget>, OverrideButtonWidget)
		/** Delegate to invoke when the button is clicked. */
		SLATE_EVENT(FOnClicked, OnButtonClicked)
		/** Delegate to invoke when selection changes. */
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged)
		/** Delegate to invoke when gathering extensions. */
		SLATE_EVENT(FOnGetExtensions, OnGetExtensions)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Create a column with a specified ColumnId */
	static typename FColumn::FArguments Column(const FName& InColumnName);

	/** Get a copy of the current selected item. */
	ItemType GetSelectedItem() const;

private:
	/** Array of data items that we are displaying. */
	TArray<ItemType> PermutationsSource;

	/** Index of the selected item in ItemsSource. It will always be valid. */
	int32 SelectedPermutationIndex;

	/** Delegate to invoke when selection changes. */
	FOnSelectionChanged OnSelectionChanged;

	/** Columns information. */
	TIndirectArray<FColumn> PropertyColumns;

	/** Box used as container for the radio button menu. */
	TSharedPtr<SHorizontalBox> ColumnContainer;

private:
	void BuildColumns(int32 StartIndex);
	void ItemSelected(int32 UniqueItemIndexSelected, int32 ColumnSelected);
	TArray<int32> GenerateItemIndexes() const;
	bool IsColumnVisible(const FColumn& Column, const TArray<int32>& UniqueItemIndexes) const;

private:
	bool IdenticalProperty(FName ColumnName, int32 LeftItemIndex, int32 RightItemIndex)
	{
		check(PermutationsSource.IsValidIndex(LeftItemIndex));
		check(PermutationsSource.IsValidIndex(RightItemIndex));
		return ItemBuilder::IdenticalProperty(ColumnName, PermutationsSource[LeftItemIndex], PermutationsSource[RightItemIndex]);
	}

	bool Less(FName ColumnName, int32 LeftItemIndex, int32 RightItemIndex)
	{
		check(PermutationsSource.IsValidIndex(LeftItemIndex));
		check(PermutationsSource.IsValidIndex(RightItemIndex));
		return ItemBuilder::Less(ColumnName, PermutationsSource[LeftItemIndex], PermutationsSource[RightItemIndex]);
	}

	FText GetLabel(FName ColumnName, int32 ItemIndex)
	{
		check(PermutationsSource.IsValidIndex(ItemIndex));
		return ItemBuilder::GetLabel(ColumnName, PermutationsSource[ItemIndex]);
	}

	FText GetTooltip(FName ColumnName, int32 ItemIndex)
	{
		check(PermutationsSource.IsValidIndex(ItemIndex));
		return ItemBuilder::GetTooltip(ColumnName, PermutationsSource[ItemIndex]);
	}
};

#include "Widgets/SMediaPermutationsSelector.inl"
