// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"

template <typename ItemType, typename ItemBuilder>
typename SMediaPermutationsSelector<ItemType, ItemBuilder>::FColumn::FArguments SMediaPermutationsSelector<ItemType, ItemBuilder>::Column(const FName& InColumnName)
{
	typename FColumn::FArguments NewArgs;
	NewArgs._ColumnName = InColumnName;
	return NewArgs;
}

template <typename ItemType, typename ItemBuilder>
ItemType SMediaPermutationsSelector<ItemType, ItemBuilder>::GetSelectedItem() const
{
	return PermutationsSource.IsValidIndex(SelectedPermutationIndex) ? PermutationsSource[SelectedPermutationIndex] : ItemType();
}

template <typename ItemType, typename ItemBuilder>
void SMediaPermutationsSelector<ItemType, ItemBuilder>::Construct(const FArguments& InArgs)
{
	PermutationsSource = InArgs._PermutationsSource;
	SelectedPermutationIndex = INDEX_NONE;
	OnSelectionChanged = InArgs._OnSelectionChanged;

	if (InArgs._SelectedPermutation.IsSet())
	{
		SelectedPermutationIndex = PermutationsSource.IndexOfByKey(InArgs._SelectedPermutation.GetValue());
	}

	// Copy all the column info from the declaration
	PropertyColumns.Empty(InArgs.Slots.Num());
	for (int32 SlotIndex = 0; SlotIndex < InArgs.Slots.Num(); ++SlotIndex)
	{
		FColumn* const Column = InArgs.Slots[SlotIndex];
		if (Column && Column->ColumnName != NAME_None)
		{
			PropertyColumns.Add(Column);
		}
	}

	SAssignNew(ColumnContainer, SHorizontalBox);

	TArray<TSharedRef<SWidget>> Extensions;
	InArgs._OnGetExtensions.ExecuteIfBound(Extensions);
	const bool bIsVisible = Extensions.Num() != 0;

	TSharedRef<SHorizontalBox> ExtensionWidgetContainer = SNew(SHorizontalBox)
		.Visibility(bIsVisible ? EVisibility::Visible : EVisibility::Collapsed);

	for (int32 Index = Extensions.Num() - 1; Index >= 0; Index--)
	{
		ExtensionWidgetContainer->AddSlot()
			.AutoWidth()
			[
				Extensions[Index]
			];
	}

	BuildColumns(0);

	ChildSlot
	[
		SNew(SBox)
		.HeightOverride(InArgs._ColumnHeight)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(1.f)
			[
				ColumnContainer.ToSharedRef()
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SNullWidget::NullWidget
				]
				+ SHorizontalBox::Slot()
				.Padding(4.f)
				.AutoWidth()
				[
					ExtensionWidgetContainer
				]
				+ SHorizontalBox::Slot()
				.Padding(4.f)
				.AutoWidth()
				[
					InArgs._OverrideButtonWidget.IsValid() ?
						InArgs._OverrideButtonWidget.ToSharedRef()
						:
						SNew(SButton)
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						.Text(NSLOCTEXT("MediaPlayerEditor", "ApplyLabel", "Apply"))
						.OnClicked(InArgs._OnButtonClicked)
				]
			]
		]
	];
}

template <typename ItemType, typename ItemBuilder>
void SMediaPermutationsSelector<ItemType, ItemBuilder>::BuildColumns(int32 StartIndex)
{
	const int32 NumberOfItems = PermutationsSource.Num();
	if (!PropertyColumns.IsValidIndex(StartIndex) || NumberOfItems == 0)
	{
		return;
	}

	if (SelectedPermutationIndex == INDEX_NONE)
	{
		SelectedPermutationIndex = 0;
		OnSelectionChanged.ExecuteIfBound(PermutationsSource[SelectedPermutationIndex]);
	}

	// Get only the valid item for that selected input
	TArray<int32> AllValidItemIndexes = GenerateItemIndexes();

	// Clean the items list. AllValidItems should contains only the valid entry up to category StartIndex.
	for (int32 ColumnIndex = 0; ColumnIndex < StartIndex; ++ColumnIndex)
	{
		const FColumn& Column = PropertyColumns[ColumnIndex];
		for (int32 ItemIndex = AllValidItemIndexes.Num() - 1; ItemIndex >= 0; --ItemIndex)
		{
			if (!IdenticalProperty(Column.ColumnName, SelectedPermutationIndex, AllValidItemIndexes[ItemIndex]))
			{
				AllValidItemIndexes.RemoveAtSwap(ItemIndex);
			}
		}
	}

	// ReBuild the items
	for (int32 ColumnIndex = StartIndex; ColumnIndex < PropertyColumns.Num(); ++ColumnIndex)
	{
		FColumn& Column = PropertyColumns[ColumnIndex];
		if (Column.Widget.IsValid())
		{
			ColumnContainer->RemoveSlot(Column.Widget.ToSharedRef());
			Column.Widget.Reset();
		}

		// Gather the unique item and remove all that do not match with the current selected item
		TArray<int32> UniqueItemsForColumnIndexes;
		const FName ColumnName = Column.ColumnName;

		for (int32 ItemIndex = AllValidItemIndexes.Num() - 1; ItemIndex >= 0; --ItemIndex)
		{
			bool bFound = false;
			for (int32 UniqueIndex : UniqueItemsForColumnIndexes)
			{
				if (IdenticalProperty(ColumnName, UniqueIndex, AllValidItemIndexes[ItemIndex]))
				{
					bFound = true;
					break;
				}
			}
			if (!bFound)
			{
				UniqueItemsForColumnIndexes.Add(AllValidItemIndexes[ItemIndex]);
			}

			if (!IdenticalProperty(ColumnName, SelectedPermutationIndex, AllValidItemIndexes[ItemIndex]))
			{
				AllValidItemIndexes.RemoveAtSwap(ItemIndex);
			}
		}

		// Sort the column items
		UniqueItemsForColumnIndexes.Sort([this, ColumnName](int32 Left, int32 Right) { return Less(ColumnName, Left, Right); });

		// Build the radio buttons
		FMenuBuilder MenuBuilder(false, nullptr);
		for (int32 UniqueItemIndex : UniqueItemsForColumnIndexes)
		{
			MenuBuilder.AddMenuEntry(
				GetLabel(ColumnName, UniqueItemIndex),
				GetTooltip(ColumnName, UniqueItemIndex),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([this, UniqueItemIndex, ColumnIndex] { ItemSelected(UniqueItemIndex, ColumnIndex); }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this, UniqueItemIndex, ColumnName] { return IdenticalProperty(ColumnName, SelectedPermutationIndex, UniqueItemIndex); })
				),
				NAME_None,
				EUserInterfaceActionType::RadioButton
			);
		}

		// Create the widget
		SAssignNew(Column.Widget, SVerticalBox)
		.Visibility_Lambda([this, Column, UniqueItemsForColumnIndexes]() { return IsColumnVisible(Column, UniqueItemsForColumnIndexes) ? EVisibility::Visible : EVisibility::Collapsed; })
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Left)
		.Padding(2, 0, 5, 0)
		[
			SNew(STextBlock)
			.Text(Column.Label)
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			MenuBuilder.MakeWidget()
		];

		// Attach the widget
		ColumnContainer->AddSlot()
		.AutoWidth()
		[
			Column.Widget.ToSharedRef()
		];
	}
}

template <typename ItemType, typename ItemBuilder>
void SMediaPermutationsSelector<ItemType, ItemBuilder>::ItemSelected(int32 UniqueItemIndex, int32 ColumnIndex)
{
	const int32 NumberOfItems = PermutationsSource.Num();
	if (!PropertyColumns.IsValidIndex(ColumnIndex) || NumberOfItems == 0)
	{
		return;
	}

	int32 PreviousSelectedItemIndex = SelectedPermutationIndex;
	SelectedPermutationIndex = INDEX_NONE;
	
	// Get only the valid item for that selected input
	TArray<int32> AllValidItemIndexes = GenerateItemIndexes();
	
	int32 MaxColumns = FMath::Min(ColumnIndex + 1, PropertyColumns.Num());
	for (int32 Index = 0; Index < MaxColumns; ++Index)
	{
		const FColumn& Column = PropertyColumns[Index];
		for (int32 ItemIndex = AllValidItemIndexes.Num() - 1; ItemIndex >= 0; --ItemIndex)
		{
			if (!IdenticalProperty(Column.ColumnName, UniqueItemIndex, AllValidItemIndexes[ItemIndex]))
			{
				AllValidItemIndexes.RemoveAtSwap(ItemIndex);
			}
		}
	}
	
	if (AllValidItemIndexes.Num() > 0)
	{
		SelectedPermutationIndex = AllValidItemIndexes[0];
	}
	
	// Try to find something that matches what we used to have.
	for (int32 Index = MaxColumns; Index < PropertyColumns.Num(); ++Index)
	{
		TArray<int32> CopiedAllValidItemIndexes = AllValidItemIndexes;

		const FColumn& Column = PropertyColumns[Index];
		for (int32 ItemIndex = AllValidItemIndexes.Num() - 1; ItemIndex >= 0; --ItemIndex)
		{
			if (!IdenticalProperty(Column.ColumnName, PreviousSelectedItemIndex, AllValidItemIndexes[ItemIndex]))
			{
				AllValidItemIndexes.RemoveAtSwap(ItemIndex);
			}
		}
	
		if (AllValidItemIndexes.Num() == 0)
		{
			if ((Index + 1) < PropertyColumns.Num())
			{
				// if there are no options, go to the next in the list
				AllValidItemIndexes = CopiedAllValidItemIndexes;
			}
			else if (CopiedAllValidItemIndexes.Num() > 0)
			{
				SelectedPermutationIndex = CopiedAllValidItemIndexes[0];
			}
		}
	}

	if (AllValidItemIndexes.Num())
	{
		// Get the first item. (RemoveAtSwap change the order.)
		SelectedPermutationIndex = AllValidItemIndexes[0];
		for (int32 IndexValue : AllValidItemIndexes)
		{
			if (IndexValue < SelectedPermutationIndex)
			{
				SelectedPermutationIndex = IndexValue;
			}
		}
	}
	else if (SelectedPermutationIndex == INDEX_NONE)
	{
		// There should always be one selected item
		SelectedPermutationIndex = UniqueItemIndex;
	}
	
	BuildColumns(ColumnIndex + 1);
		
	if (SelectedPermutationIndex != PreviousSelectedItemIndex)
	{
		OnSelectionChanged.ExecuteIfBound(PermutationsSource[SelectedPermutationIndex]);
	}
}

template <typename ItemType, typename ItemBuilder>
TArray<int32> SMediaPermutationsSelector<ItemType, ItemBuilder>::GenerateItemIndexes() const
{
	TArray<int32> Result;
	const int32 NumberOfItems = PermutationsSource.Num();
	Result.Reserve(NumberOfItems);
	for (int32 Index = 0; Index < NumberOfItems; ++Index)
	{
		Result.Add(Index);
	}
	return MoveTemp(Result);
}

template <typename ItemType, typename ItemBuilder>
bool SMediaPermutationsSelector<ItemType, ItemBuilder>::IsColumnVisible(const FColumn& Column, const TArray<int32>& UniqueItemIndexes) const
{
	bool bResult = true;
	if (Column.IsColumnVisible.IsBound())
	{
		TArray<ItemType> UniqueItems;
		UniqueItems.Empty(UniqueItemIndexes.Num());
		for (int32 ItemIndex : UniqueItemIndexes)
		{
			UniqueItems.Add(PermutationsSource[ItemIndex]);
		}

		bResult = Column.IsColumnVisible.Execute(Column.ColumnName, UniqueItems);
	}

	return bResult;
}
